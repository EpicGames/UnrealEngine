{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Landscape Patch",
	"Description": "Support for adding landscape patches- components that can be attached to meshes to affect the landscape as the mesh is repositioned.",
	"Category": "Editor",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "LandscapePatch",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		},
		{
			"Name": "LandscapePatchEditorOnly",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit"
		}
	]
}