// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/Landscape/LandscapeCommon.ush"

float3 InCenter;
float InRadius;
float InFalloff;
float InEditLayerWeightmapAlpha;

#if CIRCLE_HEIGHT_PATCH

void ApplyLandscapeCircleHeightPatch(in float4 SVPos : SV_POSITION, out float4 OutColor : SV_Target0)
{
	const float KINDA_SMALL_NUMBER = 0.0001;
	const float ClampedFalloff = max(InFalloff, KINDA_SMALL_NUMBER);
	
	int2 DestinationTextureCoordinates = floor(SVPos.xy);

	float PatchHeight = InCenter.z;
	
	float Distance = distance(DestinationTextureCoordinates, InCenter.xy);
	
	float Alpha = 1.0;
	if (Distance > InRadius)
	{
		Alpha = 1 - min((Distance - InRadius) / ClampedFalloff, 1.0);
	}

	OutColor = float4(PackHeight(PatchHeight), PackHeightAlpha(Alpha, EHEIGHTMAPALPHAFLAGS_ALPHABLEND));
}

#endif // CIRCLE_HEIGHT_PATCH

#if CIRCLE_VISIBILITY_PATCH

void ApplyLandscapeCircleVisibilityPatch(in float4 SVPos : SV_POSITION, out float4 OutColor : SV_Target0)
{
	int2 DestinationTextureCoordinates = floor(SVPos.xy);
	float Distance = distance(DestinationTextureCoordinates, InCenter.xy);

	// Apply the edit layer's weightmap alpha
	// Visibility acts a binary toggle - affects the landscape if only if alpha is greater than 0.66f
	float Alpha = 1.0f * InEditLayerWeightmapAlpha;

	OutColor = (Distance <= InRadius) ? float4(1.0f, 1.0f, Alpha, Alpha) : 0.0f;
}

#endif // CIRCLE_VISIBILITY_PATCH
