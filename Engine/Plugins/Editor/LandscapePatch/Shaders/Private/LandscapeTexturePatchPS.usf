// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "/Engine/Private/Common.ush"
#include "/Engine/Private/Landscape/LandscapeCommon.ush"

#if defined (__INTELLISENSE__)
// Uncomment the appropriate define for enabling syntax highlighting with HLSL Tools for Visual Studio : 
//#define APPLY_HEIGHT_PATCH 1
//#define OFFSET_HEIGHT_PATCH 1
//#define SIMPLE_TEXTURE_COPY 1
//#define CONVERT_TO_NATIVE_LANDSCAPE_PATCH 1
//#define CONVERT_BACK_FROM_NATIVE_LANDSCAPE_PATCH 1
//#define APPLY_WEIGHT_PATCH 1
//#define REINITIALIZE_PATCH 1
#endif // defined (__INTELLISENSE__)


#if APPLY_HEIGHT_PATCH || APPLY_WEIGHT_PATCH

// Given the falloff settings, gives the alpha to use for a pixel at the given patch UV coordinates.
float GetFalloffAlpha(float FalloffWorldMargin, float2 PatchWorldDimensions, float2 PatchUVCoordinates, float2 EdgeUVDeadBorder, bool bRectangularFalloff)
{
	static const float HALF_PI = 3.14159265f / 2;
	static const float FLOAT_TOLERANCE = 0.0001;
	
	// The falloff starts at the edges and goes inward, and shouldn't be higher than 1/2 of the smaller dimension, or
	// else it extends past the center. If it does extend past the center, we'll decrease the max alpha in the center
	// (imagine shaving off a cube with progressively flatter slopes- once you get past the center of the top, you get
	// a progressively shorter pyramid). Maybe we shouldn't even bother dealing with this case, but it's easy enough to do.
	float ClampedFalloffMargin = clamp(FalloffWorldMargin, 0, min(PatchWorldDimensions.x, PatchWorldDimensions.y) / 2);
	float bFalloffAlpha = FalloffWorldMargin > 0 ? ClampedFalloffMargin / FalloffWorldMargin : 1;
	
	// Alpha T is a parameter that we'll pass through a cos^2 function to get the alpha (to have it be smoother). It should be 
	// 0 at the places where there is no falloff (ie alpha is 1), and go to 1 at places where falloff is complete (and falloff is 0)
	float AlphaT = 1;
	
	if (bRectangularFalloff)
	{
		float2 DistancesFromEdges = PatchWorldDimensions * min(PatchUVCoordinates - EdgeUVDeadBorder, 1 - EdgeUVDeadBorder - PatchUVCoordinates);
		bool bIsInsidePatch = all(DistancesFromEdges >= -FLOAT_TOLERANCE);
		AlphaT = bIsInsidePatch ? 0 : 1;
		
		if (bIsInsidePatch && ClampedFalloffMargin > 0)
		{
			// Interpret distances as proportions of falloff margin
			DistancesFromEdges = DistancesFromEdges / ClampedFalloffMargin;
		
			if (DistancesFromEdges.x > 0 && DistancesFromEdges.x < 1.0
				&& DistancesFromEdges.y > 0 && DistancesFromEdges.y < 1.0)
			{
				AlphaT = min(1.0, length(float2(1, 1) - DistancesFromEdges));
			}
			else
			{
				AlphaT = 1 - min(1.0, min(DistancesFromEdges.x, DistancesFromEdges.y));
			}
		}
	}
	else
	{
		float OuterRadius = min(PatchWorldDimensions.x, PatchWorldDimensions.y) / 2;
		float DistanceFromCenter = length(PatchWorldDimensions * (PatchUVCoordinates - float2(0.5, 0.5)));
		float DistanceFromOuterCircle = OuterRadius - DistanceFromCenter;
		
		bool bIsInsideCircle = DistanceFromOuterCircle > 0;
		AlphaT = bIsInsideCircle ? 0 : 1;
		if (bIsInsideCircle && ClampedFalloffMargin > 0)
		{
			AlphaT = 1 - DistanceFromOuterCircle / ClampedFalloffMargin;
			AlphaT = max(0, AlphaT); // clamp to 0 so we don't go past ClampedFalloffMargin
		}
	}
	
	float Alpha = cos(AlphaT * HALF_PI);
	Alpha = Alpha * Alpha * bFalloffAlpha;
	
	return Alpha;
}

float GetChannelFromSample(uint InTextureChannel, float4 InSampledValue)
{
	switch (InTextureChannel)
	{
		case TEXTURE_CHANNEL_R:
			return InSampledValue.r;
		case TEXTURE_CHANNEL_G:
			return InSampledValue.g;
		case TEXTURE_CHANNEL_B:
			return InSampledValue.b;
		case TEXTURE_CHANNEL_A:
			return InSampledValue.a;
		case TEXTURE_CHANNEL_LUM:
			return Luminance(InSampledValue.rgb);
		default:
			return 1;
	}
}

float GetAlphaFactor(uint InAlphaMode, uint InAlphaTextureChannel, float4 InPatchTextureSampledValue, float4 InAlphaTextureSampledValue)
{
	switch (InAlphaMode)
	{
		case ALPHA_PATCH_NONE:
			return 1;
		case ALPHA_PATCH_INTERNAL_TEXTURE:
		case ALPHA_PATCH_RENDER_TARGET:
		case ALPHA_PATCH_TEXTURE_ASSET:
			return GetChannelFromSample(InAlphaTextureChannel, InAlphaTextureSampledValue);
		case ALPHA_PATCH_SRC_TEXTURE_CHANNEL:
			return GetChannelFromSample(InAlphaTextureChannel, InPatchTextureSampledValue);
		default:
			return 1;
	}
}
#endif // APPLY_HEIGHT_PATCH || APPLY_WEIGHT_PATCH

#if APPLY_HEIGHT_PATCH
Texture2D<float4> InHeightPatch;
SamplerState InHeightPatchSampler;
// Heightmap to PatchUV transform. Default patch behavior that samples the texture relative to the patch's bounds
float4x4 InHeightmapToPatchUV;
// InHeightmapToPatchUV matrix with the source texture transform (world space or relative) applied
float4x4 InHeightmapToPatchUVTransform;
float2 InPatchWorldDimensions;
float2 InEdgeUVDeadBorder;
float InFalloffWorldMargin;
// Value that corresponds to 0 (i.e. to LANDSCAPE_MID_VALUE) in the patch data
float InZeroInEncoding;
// Scale to apply to patch values relative to the value representing zero
float InHeightScale;
// An offset to apply to the resulting height, after applying the height scale
float InHeightOffset;
// Enum defining blend modes, with values set in corresponding cpp file.
uint InBlendMode;
// A combination of flags, whose positions are set in the corresponding cpp file.
uint InFlags;
// Height source texture color channel
uint InTextureChannel;
// Alpha source mode, override texture (White system texture if undefined), alpha texture color channel, and texture transform
uint InAlphaMode;
Texture2D<float4> InHeightPatchAlphaTexture;
uint InAlphaTextureChannel;
// InHeightmapToPatchUV matrix with the alpha source texture transform (world space or relative) applied
float4x4 InHeightmapAlphaToPatchUVTransform;
// Assigned edit layer's heightmap/weightmap alpha
float InEditLayerHeightmapAlpha;
float InEditLayerWeightmapAlpha;

uint ConvertPatchBlendModeToHeightAlphaFlags()
{
	switch (InBlendMode)
	{
		case ADDITIVE_MODE:
			return EHEIGHTMAPALPHAFLAGS_ADDITIVE;
		case ALPHA_BLEND_MODE:
			return (EHEIGHTMAPALPHAFLAGS_MIN | EHEIGHTMAPALPHAFLAGS_MAX);
		case MIN_MODE:
			return EHEIGHTMAPALPHAFLAGS_MIN;
		case MAX_MODE:
			return EHEIGHTMAPALPHAFLAGS_MAX;
		default:
			return EHEIGHTMAPALPHAFLAGS_NONE;
	}
}

void ApplyLandscapeTextureHeightPatch(in float4 SVPos : SV_POSITION, out float4 OutColor : SV_Target0)
{
	// Unpack our flags. The flag choices are set in FApplyLandscapeTextureHeightPatchPS::ModifyCompilationEnvironment
	// so that they match to the cpp file
	bool bRectangularFalloff = InFlags & RECTANGULAR_FALLOFF_FLAG;
	bool bInputIsPackedHeight = InFlags & INPUT_IS_PACKED_HEIGHT_FLAG;

	// We need only the 2D affine transformation that goes from landscape XY heightmap integer coordinates
	// to patch UV coordinates. SVPos is in batched merge texture coordinates here
	float4 SVPosCoordinates = float4(SVPos.xy, 0.0, 1.0);
	// Store normalized coordinates that map the patch to a [0, 1] UV space with no tiling (used for patch falloff)
	float4 PatchUVCoordinates = float4(mul(InHeightmapToPatchUV, SVPosCoordinates).xy, 0, 1);

	// Apply the HeightmapToPatch matrix with scale, offset, and rotation to convert from normalized UV to tiling space (utilizes AM_Wrap sampler)
	float2 PatchUVCoordinatesTransformed = mul(InHeightmapToPatchUVTransform, SVPosCoordinates).xy;
	float2 PatchAlphaUVCoordinatesTransformed = mul(InHeightmapAlphaToPatchUVTransform, SVPosCoordinates).xy;

	float4 PatchSampledValue = InHeightPatch.Sample(InHeightPatchSampler, PatchUVCoordinatesTransformed);
	// When using an internal texture/native packed height we enforce that the data comes from the RG channels
	// Otherwise, get the selected texture channel to use for the height
	float PatchStoredHeight = bInputIsPackedHeight ? UnpackHeight(PatchSampledValue.xy) : GetChannelFromSample(InTextureChannel, PatchSampledValue);
	
	float PatchSignedHeight = InHeightScale * (PatchStoredHeight - InZeroInEncoding) + InHeightOffset;
	
	float Alpha = GetFalloffAlpha(InFalloffWorldMargin, InPatchWorldDimensions, PatchUVCoordinates.xy, InEdgeUVDeadBorder, bRectangularFalloff);
	
	float4 PatchAlphaSampledValue = InHeightPatchAlphaTexture.Sample(InHeightPatchSampler, PatchAlphaUVCoordinatesTransformed);
	// Apply the alpha from the source height texture or use the standalone alpha texture
	Alpha *= GetAlphaFactor(InAlphaMode, InAlphaTextureChannel, PatchSampledValue, PatchAlphaSampledValue);

	// Apply the edit layer's heightmap alpha (range [-1, 1]) directly to height to support negative range (negative alpha inverts the heightmap effect)
	float OutputHeight = LANDSCAPE_MID_VALUE + (PatchSignedHeight * InEditLayerHeightmapAlpha);
	OutColor = float4(PackHeight(OutputHeight), PackHeightAlpha(Alpha, ConvertPatchBlendModeToHeightAlphaFlags()));
}
#endif // APPLY_HEIGHT_PATCH

#if OFFSET_HEIGHT_PATCH
Texture2D<float4> InHeightmap;
float InHeightOffset;

void ApplyOffsetToHeightmap(in float4 SVPos : SV_POSITION, out float4 OutColor : SV_Target0)
{
	float4 StoredVector = InHeightmap.Load(int3(SVPos.xy, 0));
	float StoredHeight = UnpackHeight(StoredVector.xy);
	OutColor = float4(PackHeight(StoredHeight + InHeightOffset), StoredVector.z, StoredVector.w);
}
#endif // OFFSET_HEIGHT_PATCH

#if CONVERT_TO_NATIVE_LANDSCAPE_PATCH
Texture2D<float4> InHeightmap;
float InZeroInEncoding;
float InHeightScale;
float InHeightOffset;

void ConvertToNativeLandscapePatch(in float4 SVPos : SV_POSITION, out float4 OutColor : SV_Target0)
{
	float4 StoredVector = InHeightmap.Load(int3(SVPos.xy, 0));
	float StoredHeight = StoredVector.x;
	// Apply the same kinds of transformations that we would apply for alpha blend mode in ApplyLandscapeTextureHeightPatch
	float InterpretedHeight = InHeightScale * (StoredHeight - InZeroInEncoding) + InHeightOffset + LANDSCAPE_MID_VALUE;
	OutColor = float4(PackHeight(InterpretedHeight), 0, StoredVector.w);
}
#endif // CONVERT_TO_NATIVE_LANDSCAPE_PATCH

#if CONVERT_BACK_FROM_NATIVE_LANDSCAPE_PATCH
Texture2D<float4> InHeightmap;
float InZeroInEncoding;
float InHeightScale;
float InHeightOffset;

void ConvertBackFromNativeLandscapePatch(in float4 SVPos : SV_POSITION, out float4 OutColor : SV_Target0)
{
	float4 StoredVector = InHeightmap.Load(int3(SVPos.xy, 0));
	float StoredHeight = UnpackHeight(StoredVector.xy);
	// Undo the transformation we do in ConvertToNativeLandscapePatch
	float PatchStoredHeight = InHeightScale ? (StoredHeight - LANDSCAPE_MID_VALUE - InHeightOffset) / InHeightScale + InZeroInEncoding
		: InZeroInEncoding;
	OutColor = float4(PatchStoredHeight, 0, 0, StoredVector.w);
}
#endif // CONVERT_BACK_FROM_NATIVE_LANDSCAPE_PATCH

#if APPLY_WEIGHT_PATCH
Texture2D<float4> InSourceWeightmap;
Texture2D<float4> InWeightPatch;
SamplerState InWeightPatchSampler;
// Weightmap to PatchUV transform. Default patch behavior that samples the texture relative to the patch's bounds
float4x4 InWeightmapToPatchUV;
// InWeightmapToPatchUV matrix with the source texture transform (world space or relative) applied
float4x4 InWeightmapToPatchUVTransform;
float2 InPatchWorldDimensions;
float2 InEdgeUVDeadBorder;
float InFalloffWorldMargin;
// Enum defining blend modes, with values set in corresponding cpp file.
uint InBlendMode;
// A combination of flags, whose positions are set in the corresponding cpp file.
uint InFlags;
// Weight source texture color channel
uint InTextureChannel;
// Alpha source mode, override texture (White system texture if undefined), alpha texture color channel, and texture transform
uint InAlphaMode;
Texture2D<float4> InWeightPatchAlphaTexture;
uint InAlphaTextureChannel;
// InWeightmapToPatchUV matrix with the alpha source texture transform (world space or relative) applied
float4x4 InWeightmapAlphaToPatchUVTransform;
// Patch edit layer's weightmap alpha
float InEditLayerWeightmapAlpha;

uint ConvertPatchBlendModeToWeightAlphaFlags()
{
	switch (InBlendMode)
	{
		case ADDITIVE_MODE:
			return EWEIGHTMAPALPHAFLAGS_ADDITIVE;
		case ALPHA_BLEND_MODE:
			return (EWEIGHTMAPALPHAFLAGS_MIN | EWEIGHTMAPALPHAFLAGS_MAX);
		case MIN_MODE:
			return EWEIGHTMAPALPHAFLAGS_MIN;
		case MAX_MODE:
			return EWEIGHTMAPALPHAFLAGS_MAX;
		default:
			return EWEIGHTMAPALPHAFLAGS_NONE;
	}
}

void ApplyLandscapeTextureWeightPatch(in float4 SVPos : SV_POSITION, out float4 OutColor : SV_Target0)
{
	// Unpack our flags. The flag choices are set in FApplyLandscapeTextureWeightPatchPS::ModifyCompilationEnvironment
	// so that they match to the cpp file
	bool bRectangularFalloff = InFlags & RECTANGULAR_FALLOFF_FLAG;

	// We need only the 2D affine transformation that goes from landscape weightmap integer coordinates
	// to patch UV coordinates. SVPos is in batched merge texture coordinates here
	float4 SVPosCoordinates = float4(SVPos.xy, 0.0, 1.0);
	// Store normalized coordinates that map the patch to a [0, 1] UV space with no tiling (used for patch falloff)
	float4 PatchUVCoordinates = float4(mul(InWeightmapToPatchUV, SVPosCoordinates).xy, 0, 1);

	// Apply the WeightmapToPatch matrix with scale, offset, and rotation to convert from normalized UV to tiling space (utilizes AM_Wrap sampler)
	float2 PatchUVCoordinatesTransformed = mul(InWeightmapToPatchUVTransform, SVPosCoordinates).xy;
	float2 PatchAlphaUVCoordinatesTransformed = mul(InWeightmapAlphaToPatchUVTransform, SVPosCoordinates).xy;

	float4 PatchSampledValue = InWeightPatch.Sample(InWeightPatchSampler, PatchUVCoordinatesTransformed);
	// Get the selected texture color channel from the sample
	float PatchWeight = GetChannelFromSample(InTextureChannel, PatchSampledValue);

	float Alpha = GetFalloffAlpha(InFalloffWorldMargin, InPatchWorldDimensions, PatchUVCoordinates.xy, InEdgeUVDeadBorder, bRectangularFalloff);

	// Apply the alpha from the source weight texture or use the standalone alpha texture
	float4 PatchAlphaSampledValue = InWeightPatchAlphaTexture.Sample(InWeightPatchSampler, PatchAlphaUVCoordinatesTransformed);
	Alpha *= GetAlphaFactor(InAlphaMode, InAlphaTextureChannel, PatchSampledValue, PatchAlphaSampledValue);
	
	// Apply the edit layer's weightmap alpha (range [0, 1])
	Alpha *= InEditLayerWeightmapAlpha;

	OutColor = float4(PackWeight(PatchWeight), PackWeightAlpha(Alpha, ConvertPatchBlendModeToWeightAlphaFlags()));
}


#endif // APPLY_WEIGHT_PATCH

#if REINITIALIZE_PATCH
Texture2D<float4> InSource;
SamplerState InSourceSampler;
float4x4 InPatchToSource;

void ReinitializePatch(in float4 SVPos : SV_POSITION, out float4 OutColor : SV_Target0)
{
	// We need only the 2D affine transformation that goes from patch integer coordinates
	// to source UV coordinates.
	float2x2 PatchToSourceRotateScale = (float2x2) InPatchToSource;
	float2 PatchToSourceTranslate = InPatchToSource._m03_m13;
	
	float2 SourceUVCoordinates = mul(PatchToSourceRotateScale, SVPos.xy) + PatchToSourceTranslate;
	OutColor = InSource.Sample(InSourceSampler, SourceUVCoordinates);

#if HEIGHT_PATCH
	// Unpacking and repacking is an easy way to make sure that we interpolate 
	// between the two bytes of height correctly.
	OutColor = float4(PackHeight(UnpackHeight(OutColor.xy)), 0, 1);
#endif
}

#endif // REINITIALIZE_PATCH

#if SIMPLE_TEXTURE_COPY
Texture2D<float4> InSource;
SamplerState InSourceSampler;
float2 InDestinationResolution;

void SimpleTextureCopy(in float4 SVPos : SV_POSITION, out float4 OutColor : SV_Target0)
{
	float2 UV = SVPos.xy / InDestinationResolution;
	OutColor = InSource.Sample(InSourceSampler, UV);
}
#endif // SIMPLE_TEXTURE_COPY