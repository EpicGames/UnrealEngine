// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class LandscapePatch : ModuleRules
{
	public LandscapePatch(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicIncludePaths.AddRange(
			new string[] {
				// ... add public include paths required here ...
			}
			);
				
		
		PrivateIncludePaths.AddRange(
			new string[] {
				// ... add other private include paths required here ...
			}
			);
			
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Landscape",
				"Engine",
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"RenderCore",
				"RHI",
				"Projects", // IPluginManager
				"Renderer",
			}
			);
			
		if (Target.bBuildEditor)
        {
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					// These are for some fix-up operations to transition patches to a newer
					// system and delete some old actors (triggered by user).
					"LevelEditor",
					"TypedElementRuntime",
					"UnrealEd", // FScopedTransaction
				}
			);
		}
	}
}
