// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeCircleHeightPatch.h"

#include "Engine/TextureRenderTarget2D.h"
#include "Engine/TextureRenderTarget2DArray.h"
#include "Landscape.h"
#include "LandscapeCircleHeightPatchPS.h"
#include "LandscapeEditLayerMergeRenderContext.h"
#include "LandscapeEditLayerTargetLayerGroup.h"
#include "LandscapeEditResourcesSubsystem.h" // ULandscapeScratchRenderTarget
#include "LandscapeInfo.h"
#include "LandscapePatchUtil.h" // GetHeightmapToWorld
#include "LandscapeUtils.h" // IsVisibilityLayer
#include "Materials/MaterialExpressionLandscapeVisibilityMask.h" 
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h" // AddCopyTexturePass
#include "RenderingThread.h"
#include "SystemTextures.h"
#include "TextureResource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LandscapeCircleHeightPatch)

#define LOCTEXT_NAMESPACE "LandscapeCircleHeightPatch"

void ULandscapeCircleHeightPatch::OnComponentCreated()
{
	Super::OnComponentCreated();

	// If we haven't been made from a copy, initialize the radius and transform of the patch
	// based on our parent.
	if (!bWasCopy)
	{
		AActor* ParentActor = GetAttachParentActor();
		if (ParentActor)
		{
			FVector Origin, BoxExtent;
			GetAttachParentActor()->GetActorBounds(false, Origin, BoxExtent);

			// Place the component at the bottom of the bounding box.
			Origin.Z -= BoxExtent.Z;
			SetWorldLocation(Origin);

			Radius = FMath::Sqrt(BoxExtent.X * BoxExtent.X + BoxExtent.Y * BoxExtent.Y);
			Falloff = Radius / 2;
		}
	}
}

#if WITH_EDITOR
UE::Landscape::EditLayers::ERenderFlags ULandscapeCircleHeightPatch::GetRenderFlags(const UE::Landscape::EditLayers::FMergeContext* InMergeContext) const
{
	using namespace UE::Landscape::EditLayers;

	return ERenderFlags::RenderMode_Recorded | ERenderFlags::BlendMode_SeparateBlend | ERenderFlags::RenderLayerGroup_SupportsGrouping;
}

bool ULandscapeCircleHeightPatch::CanGroupRenderLayerWith(TScriptInterface<ILandscapeEditLayerRenderer> InOtherRenderer) const
{
	UObject* OtherRenderer = InOtherRenderer.GetObject();
	check(OtherRenderer != nullptr);
	// Circle height patches are compatible with one another : 
	return OtherRenderer->IsA<ULandscapeCircleHeightPatch>();
}

bool ULandscapeCircleHeightPatch::RenderLayer(UE::Landscape::EditLayers::FRenderParams& RenderParams, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder)
{
	using namespace UE::Landscape::PatchUtil;
	using namespace UE::Landscape::EditLayers;

	checkf(RDGBuilderRecorder.IsRecording(), TEXT("ERenderFlags::RenderMode_Recorded means the command recorder should be recording at this point"));
	FTransform HeightmapCoordsToWorld = GetHeightmapToWorld(RenderParams.RenderAreaWorldTransform);

	ULandscapeScratchRenderTarget* LandscapeRT = RenderParams.MergeRenderContext->GetBlendRenderTargetWrite();

	const bool bIsHeightmapTarget = RenderParams.MergeRenderContext->IsHeightmapMerge();
	if (bIsHeightmapTarget)
	{
		UTextureRenderTarget2D* OutputToBlendInto = LandscapeRT->TryGetRenderTarget2D();
		check(OutputToBlendInto != nullptr);
		return ApplyCirclePatch(&RenderParams, RDGBuilderRecorder, /*bIsVisibilityLayer = */false, OutputToBlendInto->GetResource(),
			0, RenderParams.RenderAreaSectionRect.Size(), HeightmapCoordsToWorld);
	}

	// If we got to here, we're not processing a heightmap, so we only need to do anything if the 
	//  patch edits visibility.
	if (!bEditVisibility)
	{
		return false;
	}

	UTextureRenderTarget2DArray* TextureArray = LandscapeRT->TryGetRenderTarget2DArray();
	check(TextureArray != nullptr);

	int32 NumLayers = RenderParams.TargetLayerGroupLayerInfos.Num();
	check(LandscapeRT->GetEffectiveNumSlices() == NumLayers);

	bool bDidRenderSomething = false;
	for (int32 LayerIndex = 0; LayerIndex < NumLayers; ++LayerIndex)
	{
		if (UE::Landscape::IsVisibilityLayer(RenderParams.TargetLayerGroupLayerInfos[LayerIndex]))
		{
			bDidRenderSomething |= ApplyCirclePatch(&RenderParams, RDGBuilderRecorder, /*bIsVisibilityLayer = */true, TextureArray->GetResource(),
				LayerIndex, RenderParams.RenderAreaSectionRect.Size(), HeightmapCoordsToWorld);
		}
	}
	return bDidRenderSomething;
}

void ULandscapeCircleHeightPatch::BlendLayer(UE::Landscape::EditLayers::FRenderParams& RenderParams, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder)
{
	using namespace UE::Landscape::EditLayers;

	FBlendParams BlendParams;
	if (RenderParams.MergeRenderContext->IsHeightmapMerge())
	{
		BlendParams.HeightmapBlendParams.BlendMode = EHeightmapBlendMode::AlphaBlend;
	}
	// Circle height patch only supports visibility (the others are using EWeightmapBlendMode::Passthrough): 
	else if (bEditVisibility)
	{
		BlendParams.WeightmapBlendParams.Emplace(UMaterialExpressionLandscapeVisibilityMask::ParameterName, EWeightmapBlendMode::Additive);
	}

	// Then perform the generic blend : 
	RenderParams.MergeRenderContext->GenericBlendLayer(BlendParams, RenderParams, RDGBuilderRecorder);
}

bool ULandscapeCircleHeightPatch::ApplyCirclePatch(UE::Landscape::EditLayers::FRenderParams* RenderParams, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder, bool bIsVisibilityLayer,
	FTextureResource* InMergedLandscapeTextureResource, int32 LandscapeTextureSliceIndex,
	const FIntPoint& DestinationResolution, const FTransform& HeightmapCoordsToWorld, ERHIAccess OutputAccess)
{
	using namespace UE::Landscape;

	if (bEditVisibility != bIsVisibilityLayer)
	{
		return false;
	}

	const ULandscapePatchEditLayer* PatchEditLayer = GetBoundEditLayer();
	check(PatchEditLayer != nullptr);
	const float EditLayerHeightmapAlpha = PatchEditLayer->GetAlphaForTargetType(ELandscapeToolTargetType::Heightmap);
	const float EditLayerWeightmapAlpha = PatchEditLayer->GetAlphaForTargetType(ELandscapeToolTargetType::Weightmap);

	double ToHeightmapRadiusScale = GetComponentTransform().GetScale3D().X / HeightmapCoordsToWorld.GetScale3D().X;
	FVector3d CircleCenterWorld = GetComponentTransform().GetTranslation();

	// Apply patch's edit layer alpha (range [-1, 1]) to the height (negative alpha inverts heightmap effect)
	CircleCenterWorld.Z *= EditLayerHeightmapAlpha;

	FVector3d CenterInHeightmapCoordinates = HeightmapCoordsToWorld.InverseTransformPosition(CircleCenterWorld);
	float RadiusAdjustment = bExclusiveRadius ? 0 : 1;
	float HeightmapRadius = Radius * ToHeightmapRadiusScale + RadiusAdjustment;
	// TODO: This is incorrect, should not have radius adjustment here. However, need to change in a separate CL
	//  so that we can add a fixup to leave older assets unchanged.
	float HeightmapFalloff = Falloff * ToHeightmapRadiusScale + RadiusAdjustment;

	FIntRect DestinationBounds(
		FMath::Clamp(FMath::Floor(CenterInHeightmapCoordinates.X - HeightmapRadius - HeightmapFalloff),
			0, DestinationResolution.X),
		FMath::Clamp(FMath::Floor(CenterInHeightmapCoordinates.Y - HeightmapRadius - HeightmapFalloff),
			0, DestinationResolution.Y),
		FMath::Clamp(FMath::CeilToInt(CenterInHeightmapCoordinates.X + HeightmapRadius + HeightmapFalloff) + 1,
			0, DestinationResolution.X),
		FMath::Clamp(FMath::CeilToInt(CenterInHeightmapCoordinates.Y + HeightmapRadius + HeightmapFalloff) + 1,
			0, DestinationResolution.Y));

	if (DestinationBounds.Area() <= 0)
	{
		// Must be outside the landscape
		return false;
	}

	FTextureResource* OutputResource = InMergedLandscapeTextureResource;
	FString OutputResourceName = OutputResource->GetResourceName().ToString();
	check(RenderParams != nullptr);
	ULandscapeScratchRenderTarget* WriteRT = RenderParams->MergeRenderContext->GetBlendRenderTargetWrite();
	// After this point, the render cannot fail so if we're the first in our render layer group to render, we can cycle the blend render targets and 
	//  start rendering in the write one : 
	if (RenderParams->NumSuccessfulRenderLayerStepsUntilBlendLayerStep == 0)
	{
		RenderParams->MergeRenderContext->CycleBlendRenderTargets(RDGBuilderRecorder);
		WriteRT = RenderParams->MergeRenderContext->GetBlendRenderTargetWrite();
		WriteRT->Clear(RDGBuilderRecorder);
		check(WriteRT->GetCurrentState() == ERHIAccess::RTV);
	}
	OutputResource = WriteRT->GetRenderTarget()->GetResource();
	OutputResourceName = WriteRT->GetDebugName();

	auto RDGCommand =
		[ OutputResource
		, OutputResourceName
		, LandscapeTextureSliceIndex
		, DestinationBounds
		, CenterInHeightmapCoordinates
		, HeightmapRadius
		, HeightmapFalloff
		, EditLayerWeightmapAlpha
		, bEditVisibility = bEditVisibility](FRDGBuilder& GraphBuilder)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(LandscapeCircleHeightPatch);

			FRDGTextureRef DestinationTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(OutputResource->TextureRHI, TEXT("OutputTexture")));
			const TCHAR* OutputName = bEditVisibility ? TEXT("LandscapeCircleVisibilityPatchOutput") : TEXT("LandscapeCircleHeightPatchOutput");

			FLandscapeCircleHeightPatchPS::FParameters* ShaderParams = GraphBuilder.AllocParameters<FLandscapeCircleHeightPatchPS::FParameters>();
			ShaderParams->InCenter = (FVector3f)CenterInHeightmapCoordinates;
			ShaderParams->InRadius = HeightmapRadius;
			ShaderParams->InFalloff = HeightmapFalloff;
			ShaderParams->InEditLayerWeightmapAlpha = EditLayerWeightmapAlpha;
			
			ShaderParams->RenderTargets[0] = FRenderTargetBinding(DestinationTexture, ERenderTargetLoadAction::ENoAction,
				/*InMipIndex = */0, LandscapeTextureSliceIndex);

			if (bEditVisibility)
			{
				FLandscapeCircleVisibilityPatchPS::AddToRenderGraph(RDG_EVENT_NAME("RenderCircleVisibilityPatch -> %s", *OutputResourceName), GraphBuilder, ShaderParams, DestinationBounds);
			}
			else
			{
				FLandscapeCircleHeightPatchPS::AddToRenderGraph(RDG_EVENT_NAME("RenderCircleHeightPatch -> %s", *OutputResourceName), GraphBuilder, ShaderParams, DestinationBounds);
			}
		};

	// We need to specify the final state of the external texture to prevent the graph builder from transitioning it to SRVMask :
	RDGBuilderRecorder.EnqueueRDGCommand(RDGCommand, { { InMergedLandscapeTextureResource, (OutputAccess == ERHIAccess::None) ? ERHIAccess::RTV : OutputAccess} });

	return true;
}

void ULandscapeCircleHeightPatch::GetRendererStateInfo(const UE::Landscape::EditLayers::FMergeContext* InMergeContext,
	UE::Landscape::EditLayers::FEditLayerTargetTypeState& OutSupported,
	UE::Landscape::EditLayers::FEditLayerTargetTypeState& OutEnabled,
	TArray<UE::Landscape::EditLayers::FTargetLayerGroup>& OutTargetLayerGroups) const
{
	OutSupported.AddTargetType(bEditVisibility ? ELandscapeToolTargetType::Visibility
		: ELandscapeToolTargetType::Heightmap);

	// No support for weightmaps -> no need for target layer groups

	if (IsEnabled())
	{
		OutEnabled = OutSupported;
	}
}

FString ULandscapeCircleHeightPatch::GetEditLayerRendererDebugName() const
{
	return FString::Printf(TEXT("%s:%s"), *GetOwner()->GetActorNameOrLabel(), *GetName());
}

TArray<UE::Landscape::EditLayers::FEditLayerRenderItem> ULandscapeCircleHeightPatch::GetRenderItems(const UE::Landscape::EditLayers::FMergeContext* InMergeContext) const
{
	using namespace UE::Landscape::EditLayers;

	TArray<FEditLayerRenderItem> AffectedAreas;

	FTransform ComponentTransform = this->GetComponentToWorld();

	// Figure out the extents of the patch. It will be radius + falloff + an adjustment if we're
	//  trying to make the whole circle lie flat. The adjustment will be the size of one landscape
	//  quad, but to be safe we'll make it two quads in each direction.
	FVector3d LandscapeScale = InMergeContext->GetLandscape()->GetActorTransform().GetScale3D();
	FVector2D Extents(2 * FMath::Max(LandscapeScale.X, LandscapeScale.Y) + Radius + Falloff);

	FOOBox2D PatchArea(ComponentTransform, Extents);

	FInputWorldArea InputWorldArea = FInputWorldArea::CreateOOBox(PatchArea);
	FOutputWorldArea OutputWorldArea = FOutputWorldArea::CreateOOBox(PatchArea);

	FEditLayerTargetTypeState TargetInfo(InMergeContext, bEditVisibility ? ELandscapeToolTargetTypeFlags::Visibility : ELandscapeToolTargetTypeFlags::Heightmap);
	FEditLayerRenderItem Item(TargetInfo, InputWorldArea, OutputWorldArea, /*bInModifyExistingWeightmapsOnly = */false); 
	AffectedAreas.Add(Item);

	return AffectedAreas;
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
