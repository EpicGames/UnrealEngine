// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeCircleHeightPatchPS.h"

#include "PixelShaderUtils.h"
#include "LandscapeUtils.h"
#include "DataDrivenShaderPlatformInfo.h"

bool FLandscapeCircleHeightPatchPS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	// Apparently landscape requires a particular feature level
	return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5)
		&& !IsConsolePlatform(Parameters.Platform)
		&& !IsMetalMobilePlatform(Parameters.Platform);
}

void FLandscapeCircleHeightPatchPS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	UE::Landscape::ModifyShaderCompilerEnvironmentForDebug(OutEnvironment);
	OutEnvironment.SetDefine(TEXT("CIRCLE_HEIGHT_PATCH"), 1);
}

void FLandscapeCircleHeightPatchPS::AddToRenderGraph(FRDGEventName&& InRDGEventName, FRDGBuilder& GraphBuilder, FParameters* Parameters, const FIntRect& DestinationBounds)
{
	FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
	TShaderMapRef<FLandscapeCircleHeightPatchPS> PixelShader(ShaderMap);

	FPixelShaderUtils::AddFullscreenPass(
		GraphBuilder,
		ShaderMap,
		MoveTemp(InRDGEventName),
		PixelShader,
		Parameters,
		DestinationBounds);
}

void FLandscapeCircleVisibilityPatchPS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	UE::Landscape::ModifyShaderCompilerEnvironmentForDebug(OutEnvironment);
	OutEnvironment.SetDefine(TEXT("CIRCLE_VISIBILITY_PATCH"), 1);
}

void FLandscapeCircleVisibilityPatchPS::AddToRenderGraph(FRDGEventName&& InRDGEventName, FRDGBuilder& GraphBuilder, FParameters* Parameters, const FIntRect& DestinationBounds)
{
	FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
	TShaderMapRef<FLandscapeCircleVisibilityPatchPS> PixelShader(ShaderMap);

	FPixelShaderUtils::AddFullscreenPass(
		GraphBuilder,
		ShaderMap,
		MoveTemp(InRDGEventName),
		PixelShader,
		Parameters,
		DestinationBounds);
}

IMPLEMENT_GLOBAL_SHADER(FLandscapeCircleHeightPatchPS, "/Plugin/LandscapePatch/Private/LandscapeCircleHeightPatchPS.usf", "ApplyLandscapeCircleHeightPatch", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(FLandscapeCircleVisibilityPatchPS, "/Plugin/LandscapePatch/Private/LandscapeCircleHeightPatchPS.usf", "ApplyLandscapeCircleVisibilityPatch", SF_Pixel);