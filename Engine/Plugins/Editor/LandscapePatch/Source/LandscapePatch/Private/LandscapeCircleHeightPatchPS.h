// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GlobalShader.h"
#include "ShaderParameterStruct.h"

/**
 * Shader that applies a circle height patch to a landscape heightmap.
 */
class FLandscapeCircleHeightPatchPS : public FGlobalShader
{
	DECLARE_GLOBAL_SHADER(FLandscapeCircleHeightPatchPS);
	SHADER_USE_PARAMETER_STRUCT(FLandscapeCircleHeightPatchPS, FGlobalShader);

public:
	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FVector3f, InCenter)
		SHADER_PARAMETER(float, InRadius)
		SHADER_PARAMETER(float, InFalloff)
		// Texture patches apply the assigned edit layer's weightmap alpha to all weightmaps (including visibility)
		// For consistency, the circle patch similarly applies the edit layer's weightmap alpha to visibility
		SHADER_PARAMETER(float, InEditLayerWeightmapAlpha)
		RENDER_TARGET_BINDING_SLOTS() // Holds our output
	END_SHADER_PARAMETER_STRUCT()

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);
	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);

	static void AddToRenderGraph(FRDGEventName&& InRDGEventName, FRDGBuilder& GraphBuilder, FParameters* InParameters, const FIntRect& DestinationBounds);
};

/**
 * Shader that applies a circle patch to a landscape visibility layer.
 */
class FLandscapeCircleVisibilityPatchPS : public FLandscapeCircleHeightPatchPS
{
	DECLARE_GLOBAL_SHADER(FLandscapeCircleVisibilityPatchPS);

public:
	FLandscapeCircleVisibilityPatchPS() = default;
	FLandscapeCircleVisibilityPatchPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FLandscapeCircleHeightPatchPS(Initializer)
	{
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);
	static void AddToRenderGraph(FRDGEventName&& InRDGEventName, FRDGBuilder& GraphBuilder, FParameters* InParameters, const FIntRect& DestinationBounds);
};