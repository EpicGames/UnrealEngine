// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapePatchComponent.h"

#include "CoreGlobals.h" // GIsReconstructingBlueprintInstances
#include "EngineUtils.h"
#include "HAL/IConsoleManager.h" // FAutoConsoleVariableRef
#include "Landscape.h"
#include "LandscapeModule.h"
#include "LandscapePatchLogging.h"
#include "LandscapePatchManager.h"
#include "LandscapePatchUtil.h" // GetHeightmapToWorld
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#include "Modules/ModuleManager.h"
#include "PropertyPairsMap.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/Package.h"
#include "UObject/UObjectBaseUtility.h" // GetNameSafe
#include "UObject/UObjectIterator.h"

#if WITH_EDITOR
#include "ScopedTransaction.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(LandscapePatchComponent)

#define LOCTEXT_NAMESPACE "LandscapePatch"

/*
 * About binding to landscape/edit layers:
 * 
 * EditLayerGuid: the guid of the edit layer that a patch wants to affect (the edit layer should be of type
 *  ULandscapePatchEditLayer). The edit layer will hold a transient list of patches that have its guid, which
 *  it should be notified to update whenever a patch is loaded/created/etc. The edit layer theoretically does
 *  not need to be notified of *not* being pointed to, as it is able to filter its transient list whenever it
 *  processes it, but we probably still want to issue a notification so that any UI displaying the patches can
 *  update.
 * DetailPanelLayerName: the name of the layer that the user sees. Should be kept in sync with EditLayerGuid.
 * DetailPanelLayerGuid: the guid representation that the user sees. Should be kept in sync with EditLayerGuid.
 * Priority: double that determines the sorting of the patches.
 * 
 * PatchManager: deprecated object that used to hold a list of patches and apply them. This is now automatically converted
 *  to a patch edit layer which handles its patches via a transient list
 * 
 * Landscape: pointer to the landscape in which the guid is found
 * 
 * Resolving erroneous states:
 * If the guid does not point to a valid edit layer, it does not get cleared (to be cleaner in cases where an
 *  edit layer might be deleted and then the deletion undone), but the patch will not work until the guid is fixed.
 */


namespace LandscapePatchComponentLocals
{
	FString NullDetailPanelLayerName = TEXT("-Null-");

	const FText FixPatchBindingsTransactionName = LOCTEXT("FixPatchBindingsTransaction", "Fix Patch Bindings");
	const FText RebindPatchTransactionName = LOCTEXT("RebindPatchTransaction", "Rebind Patch");

#if WITH_EDITOR
	FAutoConsoleCommand CCmdFixPatchBindings(
		TEXT("LandscapePatch.FixPatchBindings"),
		TEXT("For all patches, make sure that patch is properly bound to a landscape layer."),
		FConsoleCommandDelegate::CreateLambda([]() 
	{
		const FScopedTransaction Transaction(FixPatchBindingsTransactionName);

		// Iterate through all patches
		for (TObjectIterator<ULandscapePatchComponent> It(
			/*AdditionalExclusionFlags = */RF_ClassDefaultObject,
			/*bIncludeDerivedClasses = */true,
			/*InInternalExclusionFlags = */EInternalObjectFlags::Garbage); It; ++It)
		{
			ULandscapePatchComponent* Patch = *It;
			if (!IsValid(Patch))
			{
				continue;
			}

			Patch->FixBindings();
		}
	}));

#endif
}

// Note that this is not allowed to be editor-only
ULandscapePatchComponent::ULandscapePatchComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Causes OnUpdateTransform to be called when the parent is moved. Note that this is better to do here in the
	// constructor, otherwise we'd need to do it both in OnComponentCreated and PostLoad.
	// We could keep this false if we were to register to TransformUpdated, since that gets broadcast either way.
	// TODO: Currently, neither TransformUpdated nor OnUpdateTransform are triggered when parent's transform is changed
	bWantsOnUpdateTransform = true;
}

void ULandscapePatchComponent::SetIsEnabled(bool bEnabledIn)
{
	if (bEnabledIn == bIsEnabled)
	{
		return;
	}

	Modify();
	bIsEnabled = bEnabledIn;
	RequestLandscapeUpdate();
}

#if WITH_EDITOR
void ULandscapePatchComponent::CheckForErrors()
{
	Super::CheckForErrors();

	using namespace LandscapePatchComponentLocals;

	const FText FixPatchBindingsText = LOCTEXT("FixLandscapePointerButton", "Fix patch bindings");
	
	if (!IsPatchInWorld())
	{
		return;
	}

	auto GetPackageAndActorArgs = [this]()
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("Package"), FText::FromString(*GetNameSafe(GetPackage())));
		Arguments.Add(TEXT("Actor"), FText::FromString(*GetNameSafe(GetAttachmentRootActor())));
		return Arguments;
	};

	if (EditLayerGuid.IsValid())
	{
		const ULandscapeEditLayerBase* LocalEditLayer = nullptr;
		if (!Landscape.IsValid() || Landscape->IsTemplate() || (LocalEditLayer = Landscape->GetEditLayerConst(EditLayerGuid)) == nullptr)
		{
			FMessageLog("MapCheck").Warning()
			->AddToken(FUObjectToken::Create(this))
			->AddToken(FTextToken::Create(FText::Format(LOCTEXT("GuidAndLandscapeDisagree", "The patch edit layer guid "
				"did not match the Landscape pointer. (Package: {Package}, Actor: {Actor}). "
				"Fix individually or run LandscapePatch.FixPatchBindings."), GetPackageAndActorArgs())))
			->AddToken(FActionToken::Create(FixPatchBindingsText, FText(),
				FOnActionTokenExecuted::CreateWeakLambda(this, [this]() 
			{
				const FScopedTransaction Transaction(FixPatchBindingsTransactionName);
				FixBindings();
			})));
		}
		else if (Cast<ULandscapePatchEditLayer>(LocalEditLayer) == nullptr)
		{
			FMessageLog("MapCheck").Warning()
			->AddToken(FUObjectToken::Create(this))
			->AddToken(FTextToken::Create(FText::Format(LOCTEXT("GuidIsNotPatchLayer", "The patch edit layer guid points to a "
				"layer that is not a landscape patch layer. (Package: {Package}, Actor: {Actor}). "
				"Fix individually or run LandscapePatch.FixPatchBindings."), GetPackageAndActorArgs())))
			->AddToken(FActionToken::Create(FixPatchBindingsText, FText(),
				FOnActionTokenExecuted::CreateWeakLambda(this, [this]()
			{
				const FScopedTransaction Transaction(FixPatchBindingsTransactionName);
				FixBindings();
			})));
		}
	}

	if (!EditLayerGuid.IsValid())
	{
		if (Landscape.IsValid())
		{
			FMessageLog("MapCheck").Warning()
			->AddToken(FUObjectToken::Create(this))
			->AddToken(FTextToken::Create(FText::Format(LOCTEXT("PatchOnlyHasLandscape", "The patch had a landscape "
				"but did not have an edit layer guid. (Package: {Package}, Actor: {Actor}). "
				"Fix individually or run LandscapePatch.FixPatchBindings."), GetPackageAndActorArgs())))
			->AddToken(FActionToken::Create(FixPatchBindingsText, FText(),
				FOnActionTokenExecuted::CreateWeakLambda(this, [this]() 
			{
				const FScopedTransaction Transaction(FixPatchBindingsTransactionName);
				FixBindings();
			})));
		}
		else if (IsEnabled())
		{
			FMessageLog("MapCheck").Warning()
			->AddToken(FUObjectToken::Create(this))
			->AddToken(FTextToken::Create(FText::Format(LOCTEXT("EnabledPatchNotBoundToLandscape", "Patch is enabled "
				"but is not bound to a landscape edit layer. (Package: {Package}, Actor: {Actor}). "
				"Fix individually or run LandscapePatch.FixPatchBindings."), GetPackageAndActorArgs())))
			->AddToken(FActionToken::Create(FixPatchBindingsText, FText(),
				FOnActionTokenExecuted::CreateWeakLambda(this, [this]() 
			{
				const FScopedTransaction Transaction(FixPatchBindingsTransactionName);
				FixBindings();
			})));
		}
	}
}

void ULandscapePatchComponent::OnComponentCreated()
{
	using namespace LandscapePatchComponentLocals;

	Super::OnComponentCreated();

	bWasCopy = bPropertiesCopiedIndicator;
	bPropertiesCopiedIndicator = true;

	// Doing stuff during construction script reruns is a huge pain. Avoid it.
	if (GIsReconstructingBlueprintInstances)
	{
		return;
	}

	// We're going to be binding to an edit layer, which will place us in its registered patch list with
	//  our current priority. If we're later going to be updating our priority to be the highest, then 
	//  we need to temporarily lower our priority so that we aren't accidentally the highest priority patch
	//  the edit layer sees when we query it.
	double PriorityToResetTo = Priority; // may be needed to undo the following
	if (IsPatchInWorld() && PriorityInitialization == ELandscapePatchPriorityInitialization::AcquireHighest)
	{
		Priority = TNumericLimits<double>::Lowest();
	}

	// Otherwise, bind to some edit layer
	bool bConnectionToLandscapeEstablished = false;
	if (EditLayerGuid.IsValid())
	{
		bConnectionToLandscapeEstablished = BindToEditLayer(EditLayerGuid);
	}
	if (!bConnectionToLandscapeEstablished && Landscape.IsValid())
	{
		// Minor note: the above BindToEditLayer can fail and yet still change the landscape pointer if the
		//  guid pointed to a real layer in a different landscape but of the wrong type. This behavior is
		//  probably desirable (presumably the user wanted to bind to that landscape?) but hard to say, and
		//  unlikely to come up in the first place.
		bConnectionToLandscapeEstablished = BindToLandscape(Landscape.Get());
	}
	if (!bConnectionToLandscapeEstablished && IsPatchInWorld())
	{
		BindToAnyLandscape();
	}

	// Update priority
	if (IsPatchInWorld())
	{
		switch (PriorityInitialization)
		{
		case ELandscapePatchPriorityInitialization::KeepOriginal:
			// Do nothing
			break;
		case ELandscapePatchPriorityInitialization::SmallIncrement:
			Priority += 0.01;
			if (EditLayer.IsValid())
			{
				EditLayer->NotifyOfPriorityChange(this);
			}
			break;
		case ELandscapePatchPriorityInitialization::AcquireHighest:
			if (EditLayer.IsValid())
			{
				Priority = EditLayer->GetHighestPatchPriority() + 1;
				EditLayer->NotifyOfPriorityChange(this);
			}
			else
			{
				// If we failed to get an edit layer to query, undo the setting of low priority
				//  we did above.
				Priority = PriorityToResetTo;
			}
			break;
		}
	}
}

void ULandscapePatchComponent::PostLoad()
{
	Super::PostLoad();
}

void ULandscapePatchComponent::PostEditUndo()
{
	Super::PostEditUndo();

	// Keeps all of our guid-related variables consistent
	BindToEditLayer(EditLayerGuid);

	// Makes sure we update 
	RequestLandscapeUpdate();
}

void ULandscapePatchComponent::OnComponentDestroyed(bool bDestroyingHierarchy)
{
	if (!GIsReconstructingBlueprintInstances)
	{
		if (EditLayer.IsValid())
		{
			// Notify the layer that the patch is being destroyed. Note that we are not yet marked
			//  garbage, so clear our edit layer guid so that the edit layer correctly sees us as
			//  disconnected.
			ULandscapePatchEditLayer* EditLayerToNotify = EditLayer.Get();
			ResetEditLayer();
		}
	}
	Super::OnComponentDestroyed(bDestroyingHierarchy);
}

void ULandscapePatchComponent::OnRegister()
{
	Super::OnRegister();

	if (!IsPatchInWorld())
	{
		return;
	}

	if (!GIsReconstructingBlueprintInstances && EditLayerGuid.IsValid())
	{
		BindToEditLayer(EditLayerGuid);
	}

	// TODO: We should make the invalidation conditional on whether we actually modify any relevant
	// properties by having a virtual method that compares and updates a stored hash of them.
	if (IsEnabled())
	{
		RequestLandscapeUpdate();
	}
}

void ULandscapePatchComponent::GetActorDescProperties(FPropertyPairsMap& PropertyPairsMap) const
{
	Super::GetActorDescProperties(PropertyPairsMap);

	if (Landscape.IsValid())
	{
		PropertyPairsMap.AddProperty(ALandscape::AffectsLandscapeActorDescProperty, *Landscape->GetLandscapeGuid().ToString());
	}
}

TStructOnScope<FActorComponentInstanceData> ULandscapePatchComponent::GetComponentInstanceData() const
{
	return MakeStructOnScope<FActorComponentInstanceData, FLandscapePatchComponentInstanceData>(this);
}

void ULandscapePatchComponent::OnUpdateTransform(EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport)
{
	Super::OnUpdateTransform(UpdateTransformFlags, Teleport);

	if (IsEnabled() && CanAffectLandscape())
	{
		RequestLandscapeUpdate();
	}
}

void ULandscapePatchComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	using namespace LandscapePatchComponentLocals;

	// If we're changing the owning landscape or patch manaager, there's some work we need to do to remove/add 
	// ourselves from/to the proper brush managers.
	if (PropertyChangedEvent.Property 
		&& (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ULandscapePatchComponent, Landscape)))
	{
		SetLandscape(Landscape.Get());
	}
	else if (PropertyChangedEvent.Property
		&& (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ULandscapePatchComponent, DetailPanelLayerName)))
	{
		UpdateEditLayerFromDetailPanelLayerName();
	}
	else if (PropertyChangedEvent.Property
		&& (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ULandscapePatchComponent, Priority)))
	{
		// We don't use SetPriority because that does nothing if the priority does not change, and by this point
		// the value has been updated. All we're missing is the extra notification.
		if (EditLayer.IsValid())
		{
			EditLayer->NotifyOfPriorityChange(this);
		}
	}

	// Request a landscape update as long as we're enabled, or if we just disabled ourselves.
	if (IsPatchInWorld() && (IsEnabled() || (PropertyChangedEvent.Property
		&& (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ULandscapePatchComponent, bIsEnabled)))))
	{
		RequestLandscapeUpdate();
	}

	// It is important that this super call happen after the above, because inside a blueprint actor, the call triggers a
	// rerun of the construction scripts, which will destroy the component and mess with our ability to do the above adjustments
	// properly (IsValid(this) returns false, the patch edit layer has the patch removed so it complains when we try to trigger
	// the update, etc).
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif // WITH_EDITOR

FLandscapePatchComponentInstanceData::FLandscapePatchComponentInstanceData(const ULandscapePatchComponent* Patch)
: FSceneComponentInstanceData(Patch)
{
#if WITH_EDITOR
	using namespace LandscapePatchComponentLocals;

	if (!ensure(Patch))
	{
		return;
	}

	bGaveCouldNotBindToEditLayerWarning = Patch->bGaveCouldNotBindToEditLayerWarning;
	bGaveMismatchedLandscapeWarning = Patch->bGaveMismatchedLandscapeWarning;
	bGaveMissingEditLayerGuidWarning = Patch->bGaveMissingEditLayerGuidWarning;

	EditLayerGuid = Patch->EditLayerGuid;
	Priority = Patch->Priority;
#endif
}

#if WITH_EDITOR
// Called before/after rerunning construction scripts (when patch is part of a blueprint) to carry over extra data.
void ULandscapePatchComponent::ApplyComponentInstanceData(FLandscapePatchComponentInstanceData* ComponentInstanceData, 
	ECacheApplyPhase CacheApplyPhase)
{
	using namespace LandscapePatchComponentLocals;

	if (CacheApplyPhase == ECacheApplyPhase::PostUserConstructionScript)
	{
		// Avoid stomping user construction script changes.
		return;
	}

	if (!ComponentInstanceData)
	{
		return;
	}

	bGaveCouldNotBindToEditLayerWarning = ComponentInstanceData->bGaveCouldNotBindToEditLayerWarning;
	bGaveMismatchedLandscapeWarning = ComponentInstanceData->bGaveMismatchedLandscapeWarning;
	bGaveMissingEditLayerGuidWarning = ComponentInstanceData->bGaveMissingEditLayerGuidWarning;

	EditLayerGuid = ComponentInstanceData->EditLayerGuid;
	Priority = ComponentInstanceData->Priority;

	BindToEditLayer(EditLayerGuid);

	bInstanceDataApplied = true;
	if (bDeferUpdateRequestUntilInstanceData)
	{
		RequestLandscapeUpdate();
		bDeferUpdateRequestUntilInstanceData = false;
	}
}
#endif // WITH_EDITOR

void ULandscapePatchComponent::SetLandscape(ALandscape* NewLandscape)
{
#if WITH_EDITOR

	using namespace LandscapePatchComponentLocals;

	Modify(false);
	Landscape = NewLandscape;

	if (!NewLandscape)
	{
		ResetEditLayer();
		return;
	}

	if (!BindToLandscape(NewLandscape))
	{
		UE_LOGF(LogLandscapePatch, Warning, "Unable to bind to given landscape (does it have edit layers enabled?).");
	}
#endif // WITH_EDITOR
}

// Deprecated
void ULandscapePatchComponent::SetPatchManager(ADEPRECATED_LandscapePatchManager* NewPatchManager)
{}

#if WITH_EDITOR
bool ULandscapePatchComponent::IsPatchPreview()
{
	return GetOwner() && GetOwner()->bIsEditorPreviewActor;
}
#endif

// Deprecated
ADEPRECATED_LandscapePatchManager* ULandscapePatchComponent::GetPatchManager() const
{
	return nullptr;
}

void ULandscapePatchComponent::RequestLandscapeUpdate(bool bInUserTriggeredUpdate /* = false */)
{
	// TODO: Once we're sure that the bool parameter is not necessary, we should say so in the function
	// header. Might not be able to remove safely since it's blueprint callable...

	using namespace LandscapePatchComponentLocals;

#if WITH_EDITOR
	// Note that aside from the usual guard against doing things in the blueprint editor, the check of WorldType
	// inside this call also prevents us from doing the request while cooking, where WorldType is set to Inactive. Otherwise
	// we might issue warnings below.
	if (!IsPatchInWorld())
	{
		return;
	}

	// If we get a request for a landscape during rerunning construction scripts before applying instance data,
	// defer that request until we've applied the instance data. If we don't, then the below booleans that are
	// meant to stop spamming the log will not work, since they get carried over via instance data.
	if (GIsReconstructingBlueprintInstances && !bInstanceDataApplied)
	{
		bDeferUpdateRequestUntilInstanceData = true;
		return;
	}

	// Otherwise, we work via Guid registration.
	if (!EditLayer.IsValid() && EditLayerGuid.IsValid())
	{
		BindToEditLayer(EditLayerGuid);
	}

	if (EditLayer.IsValid())
	{
		ResetWarnings();
		EditLayer->RequestLandscapeUpdate();
	}
	else if (EditLayerGuid.IsValid())
	{
		// We have a guid, but weren't able to bind to it
		if (!bGaveCouldNotBindToEditLayerWarning)
		{
			UE_LOGF(LogLandscapePatch, Warning, "Could not find an edit layer with the given guid for the patch. "
				"(Package: %ls",
				*GetNameSafe(GetPackage()));
			bGaveCouldNotBindToEditLayerWarning = true;
		}
	}
	else
	{
		// We didn't even have a guid
		if (!bGaveMissingEditLayerGuidWarning)
		{
			UE_LOGF(LogLandscapePatch, Warning, "Patch is not bound to an edit layer. Set the landscape and edit layer guid or run LandscapePatch.FixPatchBindings."
				"on the patch. (Package: %ls)",
				*GetNameSafe(GetPackage()));
			bGaveMissingEditLayerGuidWarning = true;
		}
	}
#endif
}

bool ULandscapePatchComponent::IsPatchInWorld() const
{
	UWorld* World = GetWorld();
	return !IsTemplate() && IsValidChecked(this) && IsValid(World) && World->WorldType == EWorldType::Editor;
}

#if WITH_EDITOR
void ULandscapePatchComponent::ResetWarnings()
{
	bGaveCouldNotBindToEditLayerWarning = false;
	bGaveMismatchedLandscapeWarning = false;
	bGaveMissingEditLayerGuidWarning = false;
}

// Safe to do if already bound to the given layer, and safe to do for templates
//  Changes the landscape pointer only if the guid points to a layer, even
//  if the layer is of the wrong type. Changes all guid-related variables regardless of whether 
//  the guid points to a valid layer or not. EditLayerGuid will be valid only if the guid points
//  to a layer of an appropriate type (in which case binding is considered successful). Binding
//  to null guid is same as a reset, and is considered successful.
bool ULandscapePatchComponent::BindToEditLayer(const FGuid& Guid)
{
	using namespace LandscapePatchComponentLocals;

	if (!Guid.IsValid())
	{
		ResetEditLayer();
		return true;
	}

	Modify(false);
	ULandscapePatchEditLayer* PreviousEditLayer = EditLayer.Get();

	ULandscapeEditLayerBase* NewEditLayer = nullptr;

	// See if the layer is in our current landscape
	if (Landscape.Get())
	{
		NewEditLayer = Landscape->GetEditLayer(Guid);
		if (!NewEditLayer && !bGaveMismatchedLandscapeWarning)
		{
			UE_LOGF(LogLandscapePatch, Warning, "Mismatch between landscape and layer Guid in patch.");
			bGaveMismatchedLandscapeWarning = true;
		}
	}
	
	// If the layer wasn't in the current landscape, see if it's in some other landscape
	if (!NewEditLayer)
	{
		UWorld* World = GetWorld();
		if (ensure(World))
		{
			for (TActorIterator<ALandscape> LandscapeIterator(World); LandscapeIterator; ++LandscapeIterator)
			{
				NewEditLayer = LandscapeIterator->GetEditLayer(Guid);
				if (NewEditLayer)
				{
					// Found!
					// Note that the layer we found might be of an incorrect type, so it's arguable whether landscape
					//  pointer should change yet. But probably should.
					Landscape = *LandscapeIterator;
					break;
				}
			}
		}
	}

	if (EditLayerGuid != Guid)
	{
		Modify();  // Changing the guid, so mark the package dirty.
	}

	EditLayer = Cast<ULandscapePatchEditLayer>(NewEditLayer);
	EditLayerGuid = Guid;
	DetailPanelLayerGuid = Guid.ToString();
	DetailPanelLayerName = 
		EditLayer.IsValid()
		? EditLayer->GetName().ToString()
		: TEXT("-Layer Not Found-");

	if (PreviousEditLayer && EditLayer != PreviousEditLayer)
	{
		PreviousEditLayer->NotifyOfPatchRemoval(this);
		PreviousEditLayer->RequestLandscapeUpdate();
	}

	if (EditLayer.IsValid() && IsPatchInWorld())
	{
		// This is safe to do even if we are already registered
		EditLayer->RegisterPatchForEditLayer(this);
	}

	return EditLayer.IsValid();
}

// Does not affect landscape pointer
void ULandscapePatchComponent::ResetEditLayer()
{
	using namespace LandscapePatchComponentLocals;

	ULandscapePatchEditLayer* PreviousLayer = EditLayer.Get();

	Modify(false);

	EditLayerGuid.Invalidate();
	DetailPanelLayerGuid = EditLayerGuid.ToString();
	DetailPanelLayerName = NullDetailPanelLayerName;
	EditLayer = nullptr;

	if (PreviousLayer)
	{
		PreviousLayer->NotifyOfPatchRemoval(this);
		PreviousLayer->RequestLandscapeUpdate();
	}
}

// If we have an edit layer guid in the given landscape, binds to that edit layer. Otherwise,
// looks for an edit layer to bind to in that landscape.
bool ULandscapePatchComponent::BindToLandscape(ALandscape* LandscapeIn)
{
	using namespace LandscapePatchComponentLocals;
	
	Modify(false);
	Landscape = LandscapeIn;

	if (!LandscapeIn)
	{
		ResetEditLayer();
		return true;
	}

	if (LandscapeIn->IsTemplate())
	{
		ResetEditLayer();
		return false;
	}

	const ULandscapeEditLayerBase* LocalEditLayer = nullptr;
	const ULandscapePatchEditLayer* PatchEditLayer = nullptr;

	if (EditLayerGuid.IsValid())
	{
		// See if this layer is here and of the appropriate type
		LocalEditLayer = LandscapeIn->GetEditLayerConst(EditLayerGuid);
		PatchEditLayer = LocalEditLayer ? Cast<ULandscapePatchEditLayer>(LocalEditLayer) : nullptr;
	}
	if (!PatchEditLayer)
	{
		// See if we have any layer of the appropriate type
		LocalEditLayer = LandscapeIn->FindEditLayerOfTypeConst(ULandscapePatchEditLayer::StaticClass());
		PatchEditLayer = LocalEditLayer ? Cast<ULandscapePatchEditLayer>(LocalEditLayer) : nullptr;
	}

	// If we couldn't find an appropriate layer, make a new one
	if (!PatchEditLayer && IsPatchInWorld() 
		// We don't want to create a layer if this is the preview, because that will make the layer insertion
		// not transactable, and we're not going to delete the layer if we don't complete the drop.
		&& !IsPatchPreview())
	{
		const FName PatchLayerName = LandscapeIn->GenerateUniqueLayerName(
			*ULandscapePatchEditLayer::StaticClass()->GetDefaultObject<ULandscapePatchEditLayer>()->GetDefaultName());

		ILandscapeModule& LandscapeModule = FModuleManager::GetModuleChecked<ILandscapeModule>("Landscape");
		int32 LayerIndex = LandscapeModule.GetLandscapeEditorServices()->GetOrCreateEditLayer(PatchLayerName, LandscapeIn, ULandscapePatchEditLayer::StaticClass());
		LocalEditLayer = LandscapeIn->GetEditLayerConst(LayerIndex);

		PatchEditLayer = LocalEditLayer ? Cast<ULandscapePatchEditLayer>(LocalEditLayer) : nullptr;
	}

	if (!PatchEditLayer)
	{
		// This happens if we failed to find a layer and either the patch was a preview, or IsPatchInWorld was false
		// (the latter has happened while doing a SaveAs on a level).
		ResetEditLayer();
		return false;
	}

	return BindToEditLayer(LocalEditLayer->GetGuid());
}

bool ULandscapePatchComponent::BindToAnyLandscape()
{
	// While we typically allow the bind functions to operate on templates so that all members
	//  of the template can be initialized coherently, we don't want to accidentally bind
	//  templates to a landscape automatically.
	// So, early out in this particular function.
	if (!IsPatchInWorld())
	{
		return false;
	}

	UWorld* World = GetWorld();
	if (!ensure(World))
	{
		return false;
	}

	for (TActorIterator<ALandscape> LandscapeIterator(World); LandscapeIterator; ++LandscapeIterator)
	{
		if (BindToLandscape(*LandscapeIterator))
		{
			return true;
		}
	}

	return false;
}

void ULandscapePatchComponent::UpdateEditLayerFromDetailPanelLayerName()
{
	using namespace LandscapePatchComponentLocals;

	if (!Landscape.IsValid())
	{
		return;
	}

	const ULandscapeEditLayerBase* CurrentEditLayer = Landscape->GetEditLayerConst(*DetailPanelLayerName);
	SetEditLayerGuid(CurrentEditLayer ? CurrentEditLayer->GetGuid() : FGuid());
}

void ULandscapePatchComponent::FixBindings()
{
	Modify(false);

	if (EditLayerGuid.IsValid())
	{
		if (BindToEditLayer(EditLayerGuid))
		{
			return;
		}

		// Otherwise, clear out the invalid guid
		UE_LOGF(LogLandscapePatch, Warning, "Could not find patch edit layer with guid: %ls", *EditLayerGuid.ToString());
		ResetEditLayer();
	}

	// If we got here, we don't have an edit layer 
	if (Landscape.IsValid())
	{
		if (BindToLandscape(Landscape.Get()))
		{
			return;
		}
	}

	// At this point there's not much we can do to templates because we don't let them create
	//  edit layers or bind to random landscapes (note: not actually sure that this function
	//  would ever be called on templates... but we'll be safe)
	if (!IsPatchInWorld())
	{
		return;
	}

	// Try to bind to any landscape.
	Landscape = nullptr;
	if (IsEnabled() && !BindToAnyLandscape())
	{
		UE_LOGF(LogLandscapePatch, Warning, "Unable to find landscape with edit layers enabled.");
	}
}

void ULandscapePatchComponent::NotifyOfBoundLayerDeletion(ULandscapePatchEditLayer* LayerIn)
{
	ResetEditLayer();
}
#endif

void ULandscapePatchComponent::SetPriority(double PriorityIn)
{
#if WITH_EDITOR
	if (Priority == PriorityIn)
	{
		return;
	}

	Modify();
	Priority = PriorityIn;
	if (EditLayer.IsValid())
	{
		EditLayer->NotifyOfPriorityChange(this);
		EditLayer->RequestLandscapeUpdate();
	}
#endif
}

void ULandscapePatchComponent::SetEditLayerGuid(const FGuid& GuidIn)
{
#if WITH_EDITOR
	Modify(false);
	BindToEditLayer(GuidIn);
	ResetWarnings();
#endif
}

bool ULandscapePatchComponent::AssignToLandscape(ALandscape* InLandscape, const FName& InEditLayerName)
{
	bool bSuccess = false;

#if WITH_EDITOR
	Modify(false);
	bSuccess = BindToLandscape(InLandscape);
	if (bSuccess && (InLandscape != nullptr))
	{
		// Passing an empty name is the equivalent of un-assigning the edit layer, it's considered valid : 
		if (InEditLayerName.IsNone())
		{
			bSuccess = BindToEditLayer(FGuid());
		}
		else
		{
			// Validate that the edit layer name corresponds to a valid patch edit layer and spew errors if not : 
			const ULandscapeEditLayerBase* LocalEditLayer = InLandscape->GetEditLayerConst(InEditLayerName);
			if (LocalEditLayer != nullptr)
			{
				const ULandscapePatchEditLayer* PatchEditLayer = Cast<const ULandscapePatchEditLayer>(LocalEditLayer);
				if (PatchEditLayer != nullptr)
				{
					// All good, we can try to attach to that edit layer : 
					bSuccess = BindToEditLayer(LocalEditLayer->GetGuid());
				}
				else
				{
					UE_LOGF(LogLandscapePatch, Warning, "Edit layer with the given name (%ls) on landscape %ls is not a patch edit layer (%ls). "
						"(Package: %ls",
						*InEditLayerName.ToString(),
						*InLandscape->GetActorLabel(),
						*LocalEditLayer->GetClass()->GetName(),
						*GetNameSafe(GetPackage()));

					// Un-assign the edit layer so that the function call effectively changes the edit layer according to the error :
					BindToEditLayer(FGuid());
					bSuccess = false;
				}
			}
			else
			{
				UE_LOGF(LogLandscapePatch, Warning, "Could not find an edit layer with the given name (%ls) on landscape %ls for the patch. "
					"(Package: %ls",
					*InEditLayerName.ToString(),
					*InLandscape->GetActorLabel(),
					*GetNameSafe(GetPackage()));

				// Un-assign the edit layer so that the function call effectively changes the edit layer according to the error :
				BindToEditLayer(FGuid());
				bSuccess = false;
			}
		}
	}
#endif

	return bSuccess;
}

uint32 ULandscapePatchComponent::GetFullNameHash() const
{
	// TODO: If needed, we can cache this value, but then we need to update it in
	// PostLoad, creation, carry it over construction script reruns, and update it
	// if the name changes.
	return GetTypeHash(GetFullName());;
}

// Provides a dropdown of available guids in the detail panel.
const TArray<FString> ULandscapePatchComponent::GetLayerOptions()
{
	using namespace LandscapePatchComponentLocals;

	TArray<FString> Options{ NullDetailPanelLayerName };

#if WITH_EDITOR
	// Get the layer names from our landscape
	if (Landscape.IsValid())
	{
		for (const ULandscapeEditLayerBase* CurrentEditLayer : Landscape->GetEditLayersConst())
		{
			if (CurrentEditLayer->IsA<ULandscapePatchEditLayer>())
			{
				Options.Add(CurrentEditLayer->GetName().ToString());
			}
		}
	}
#endif

	return Options;
}

FTransform ULandscapePatchComponent::GetLandscapeHeightmapCoordsToWorld() const
{
	if (Landscape.IsValid())
	{
		return UE::Landscape::PatchUtil::GetHeightmapToWorld(Landscape->GetTransform());
	}
	return FTransform::Identity;
}

#undef LOCTEXT_NAMESPACE

