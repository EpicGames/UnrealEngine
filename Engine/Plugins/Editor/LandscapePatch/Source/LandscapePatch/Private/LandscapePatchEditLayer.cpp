// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapePatchEditLayer.h"

#include "Algo/AnyOf.h"
#include "Algo/Sort.h"
#include "Landscape.h"
#include "LandscapeEditLayerMergeRenderContext.h"
#include "LandscapePatchLogging.h"
#include "LandscapePatchUtil.h"
#include "CoreGlobals.h" // UE::GetIsEditorLoadingPackage()

#include "LandscapePatchComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LandscapePatchEditLayer)

#if WITH_EDITOR
namespace LandscapePatchEditLayerLocals
{
	auto PatchSortPredicateRaw = [](ULandscapePatchComponent* A, ULandscapePatchComponent* B)
	{
		if (!ensure(A && B))
		{
			return false;
		}

		// If priorities are different, sort on priority
		return A->GetPriority() != B->GetPriority() ? A->GetPriority() < B->GetPriority()

			// If priorities are the same, use the full name hash. The comparison is meaningless, but gives
			//  a deterministic ordering across runs, regardless of registration order.
			: A->GetFullNameHash() != B->GetFullNameHash() ? A->GetFullNameHash() < B->GetFullNameHash()

			// Hopefully we don't actually have to do full name string comparison, but that's the fallback.
			: A->GetFullName() < B->GetFullName();
	};

	auto PatchSortPredicate = [](const TSoftObjectPtr<ULandscapePatchComponent>& ASoft, const TSoftObjectPtr<ULandscapePatchComponent>& BSoft)
	{
		return PatchSortPredicateRaw(ASoft.Get(), BSoft.Get());
	};
}

void ULandscapePatchEditLayer::RegisterPatchForEditLayer(ULandscapePatchComponent* Patch)
{
	using namespace LandscapePatchEditLayerLocals;

	if (!ShouldPatchBeIncludedInList(Patch))
	{
		return;
	}

	// See if we already have the patch
	int32* FoundIndex = PatchToIndex.Find(Patch);
	if (FoundIndex)
	{
		if (!ensureMsgf(RegisteredPatches.IsValidIndex(*FoundIndex) && RegisteredPatches[*FoundIndex] == Patch,
			TEXT("LandscapePatchEditLayer: PatchToIndex is expected to match RegisteredPatces")))
		{
			bPatchListDirty = true;
			FoundIndex = nullptr;
		}
	}

	if (FoundIndex)
	{
		return;
	}

	Modify();

	// See where this patch goes. If the list is up to date, we can get the actual insertion index
	//  via a binary search. Otherwise we can just put on the end since we will resort anyway.
	int32 InsertionIndex = RegisteredPatches.Num();
	if (!bPatchListDirty)
	{
		InsertionIndex = GetInsertionIndex(Patch, bPatchListDirty);
	}
	
	RegisteredPatches.Insert(Patch, InsertionIndex);

	// Update index map for this and all patches forward
	for (int32 i = InsertionIndex; i < RegisteredPatches.Num(); ++i)
	{
		PatchToIndex.Add(RegisteredPatches[i], i);
	}

	if (Patch->IsEnabled() && Patch->CanAffectLandscape())
	{
		RequestLandscapeUpdate();
	}

	UpdateHighestKnownPriority();
}

void ULandscapePatchEditLayer::NotifyOfPatchRemoval(ULandscapePatchComponent* Patch)
{
	using namespace LandscapePatchEditLayerLocals;

	if (!ensure(Patch))
	{
		return;
	}

	if (!Patch->IsPatchInWorld() && ensure(!PatchToIndex.Contains(Patch)))
	{
		return;
	}

	// If we're being notified of removal, we expect that the patch doesn't point to our layer
	//  or is otherwise legitimately not supposed to be in our list.
	ensure(!ShouldPatchBeIncludedInList(Patch));

	// See if we have this patch
	int32 RemovedIndex = 0;
	if (!PatchToIndex.RemoveAndCopyValue(Patch, RemovedIndex))
	{
		UE_LOGF(LogLandscapePatch, Warning, "LandscapePatchEditLayer: Received NotifyOfPatchRemoval call for an unregistered patch.");
		return;
	}

	ON_SCOPE_EXIT { RequestLandscapeUpdate(); };

	if (!ensureMsgf(RegisteredPatches.IsValidIndex(RemovedIndex) && RegisteredPatches[RemovedIndex] == Patch, 
		TEXT("LandscapePatchEditLayer: PatchToIndex is expected to match RegisteredPatces")))
	{
		bPatchListDirty = true;
		return;
	}

	Modify();
	RegisteredPatches.RemoveAt(RemovedIndex);

	// Update forward indices
	for (int32 i = RemovedIndex; i < RegisteredPatches.Num(); ++i)
	{
		PatchToIndex.Add(RegisteredPatches[i], i);
	}

	UpdateHighestKnownPriority();
}

void ULandscapePatchEditLayer::NotifyOfPriorityChange(ULandscapePatchComponent* Patch)
{
	using namespace LandscapePatchEditLayerLocals;

	if (!ensure(Patch) || !Patch->IsPatchInWorld())
	{
		return;
	}

	TSoftObjectPtr<ULandscapePatchComponent> PatchSoftPtr = Patch;
	int32* OriginalIndexPtr = PatchToIndex.Find(PatchSoftPtr);
	if (!OriginalIndexPtr)
	{
		UE_LOGF(LogLandscapePatch, Warning, "LandscapePatchEditLayer: Received NotifyOfPriorityChange call for an unregistered patch.");
		return;
	}

	ON_SCOPE_EXIT{ RequestLandscapeUpdate(); };

	if (bPatchListDirty)
	{
		// If patch list is dirty, we'll be resorting anyway, so no adjustment is needed right now.
		return;
	}

	int32 OriginalIndex = *OriginalIndexPtr;
	if (!ensureMsgf(RegisteredPatches[OriginalIndex] == Patch, TEXT("LandscapePatchEditLayer: PatchToIndex is expected to match RegisteredPatces")))
	{
		bPatchListDirty = true;
		return;
	}

	// See if the patch is already in the proper place. Note that we only need to consider priority because only
	//  priority changed (not the patch full name).
	double Priority = Patch->GetPriority();
	bool bPreviousPatchIsEqualOrLess = OriginalIndex == 0 
		|| (RegisteredPatches[OriginalIndex - 1].IsValid() && RegisteredPatches[OriginalIndex - 1]->GetPriority() <= Priority);
	bool bNextPatchIsEqualOrMore = OriginalIndex == RegisteredPatches.Num() - 1
		|| (RegisteredPatches[OriginalIndex + 1].IsValid() && RegisteredPatches[OriginalIndex + 1]->GetPriority() >= Priority);
	if (bPreviousPatchIsEqualOrLess && bNextPatchIsEqualOrMore)
	{
		return;
	}

	Modify();
	RegisteredPatches.RemoveAt(OriginalIndex);
	int32 InsertionIndex = GetInsertionIndex(Patch, bPatchListDirty);
	RegisteredPatches.Insert(PatchSoftPtr, InsertionIndex);
	
	// Update all the indices that changed
	int32 MinIndex = FMath::Min(OriginalIndex, InsertionIndex);
	int32 MaxIndex = FMath::Max(OriginalIndex, InsertionIndex);
	for (int32 i = MinIndex; i <= MaxIndex; ++i)
	{
		PatchToIndex.Add(RegisteredPatches[i], i);
	}

	UpdateHighestKnownPriority();
}

void ULandscapePatchEditLayer::UpdatePatchListIfDirty()
{
	if (bPatchListDirty)
	{
		UpdatePatchList();
	}
}

void ULandscapePatchEditLayer::UpdateHighestKnownPriority()
{
	if (!bPatchListDirty)
	{
		if (RegisteredPatches.Num() == 0)
		{
			HighestKnownPriority = PATCH_PRIORITY_BASE;
		}
		else if (RegisteredPatches.Last().IsValid())
		{
			HighestKnownPriority = FMath::Max(RegisteredPatches.Last()->GetPriority(), PATCH_PRIORITY_BASE);
		}
		// If the last patch was invalid, then it seems likely that multiple patches managed
		//  to become invalid at the same time, and we haven't yet removed the last one while
		//  processing the NotifyOfPatchRemoval call for a previous one. 
		// There are a few ways we could handle the situation, but for now we will just leave
		//  the highest priority unchanged, under the assumption that it will be updated in an
		//  upcoming NotifyOfPatchRemoval call, when that patch is properly removed.
	}
}

void ULandscapePatchEditLayer::UpdatePatchList()
{
	using namespace LandscapePatchEditLayerLocals;

	// Filter out any patches that are no longer associated with this layer
	RegisteredPatches.RemoveAll([this](TSoftObjectPtr<ULandscapePatchComponent> PatchSoft) 
	{
		ULandscapePatchComponent* Patch = PatchSoft.Get();
		return !ShouldPatchBeIncludedInList(Patch);
	});

	// Sort by priority
	Algo::Sort(RegisteredPatches, PatchSortPredicate);

	// Update index lookup table
	PatchToIndex.Reset();
	for (int32 i = 0; i < RegisteredPatches.Num(); ++i)
	{
		PatchToIndex.Add(RegisteredPatches[i].Get(), i);
	}

	bPatchListDirty = false;
	UpdateHighestKnownPriority();
}

double ULandscapePatchEditLayer::GetHighestPatchPriority()
{
	return HighestKnownPriority;
}
#endif // WITH_EDITOR

void ULandscapePatchEditLayer::GetRenderDependencies(TSet<UObject*>& OutDependencies) const
{
	Super::GetRenderDependencies(OutDependencies);

#if WITH_EDITOR
	for (const TSoftObjectPtr<ULandscapePatchComponent>& PatchSoft : RegisteredPatches)
	{
		ULandscapePatchComponent* Patch = PatchSoft.Get();
		if (ShouldPatchBeIncludedInList(Patch))
		{
			Patch->GetRenderDependencies(OutDependencies);
		}
	}
#endif
}

void ULandscapePatchEditLayer::OnLayerRemoved()
{
#if WITH_EDITOR
	Modify();
	// TODO: If we end up keeping this pointer, it should probably be reset in the base class implementation
	// of OnLayerRemoved.
	OwningLandscape.Reset();

	// Iterate through a copy so that patches can deregister themselves in NotifyOfBoundLayerDeletion
	//  without messing up our iteration.
	TArray<TSoftObjectPtr<ULandscapePatchComponent>> PatchesCopy = RegisteredPatches;
	for (TSoftObjectPtr<ULandscapePatchComponent> PatchSoft : PatchesCopy)
	{
		ULandscapePatchComponent* Patch = PatchSoft.Get();
		if (ShouldPatchBeIncludedInList(Patch))
		{
			Patch->NotifyOfBoundLayerDeletion(this);
		}
	}
#endif
}

bool ULandscapePatchEditLayer::SupportsTargetType(ELandscapeToolTargetType InType) const
{
	return InType != ELandscapeToolTargetType::Invalid;
}

#if WITH_EDITOR
// Called in batched merge path to apply the patches
TArray<UE::Landscape::EditLayers::FEditLayerRendererState> ULandscapePatchEditLayer::GetEditLayerRendererStates(const UE::Landscape::EditLayers::FMergeContext* InMergeContext)
{
	UpdatePatchListIfDirty();

	TArray<FEditLayerRendererState> RendererStates;
	RendererStates.Reserve(RegisteredPatches.Num());
	for (TSoftObjectPtr<ULandscapePatchComponent>& PatchSoft : RegisteredPatches)
	{
		ULandscapePatchComponent* Patch = PatchSoft.Get();
		if (!Patch)
		{
			continue;
		}

		FEditLayerRendererState& RendererState = RendererStates.Emplace_GetRef(InMergeContext, Patch);
		if (InMergeContext->ShouldSkipProceduralRenderers() || !Patch->IsEnabled())
		{
			RendererState.DisableTargetTypeMask(ELandscapeToolTargetTypeFlags::All);
		}
	}

	return RendererStates;
}

void ULandscapePatchEditLayer::PostEditUndo()
{
	Super::PostEditUndo();

	UpdatePatchList();
}

void ULandscapePatchEditLayer::RequestLandscapeUpdate(bool bInUserTriggered /* = false */)
{
	if (!OwningLandscape.IsValid())
	{
		return;
	}

	// TODO: Consider passing a parameter down to say when we're not updating height, only weights, when that is the case
	OwningLandscape->RequestLayersContentUpdateForceAll(ELandscapeLayerUpdateMode::Update_All, bInUserTriggered && !UE::GetIsEditorLoadingPackage());
}

bool ULandscapePatchEditLayer::ShouldPatchBeIncludedInList(const ULandscapePatchComponent* Patch) const
{
	return Patch && Patch->IsPatchInWorld() && Patch->GetEditLayerGuid() == GetGuid();
}

// Attempt binary search to find the insertion index for a patch. Returns end of array if an invalid patch is sampled.
int32 ULandscapePatchEditLayer::GetInsertionIndex(ULandscapePatchComponent* Patch, bool& bFoundInvalidPatchOut) const
{
	using namespace LandscapePatchEditLayerLocals;

	// This is a copy of BinarySearch.h::UpperBoundInternal, except that we have to check for validity
	//  of our check values and exit early if we find an invalid one.

	if (!ensure(Patch))
	{
		return INDEX_NONE;
	}

	if (bPatchListDirty)
	{
		bFoundInvalidPatchOut = true;
		return RegisteredPatches.Num();
	}

	// Current start of sequence to check
	int32 Start = 0;
	// Size of sequence to check
	int32 Size = RegisteredPatches.Num();

	while (Size > 0)
	{
		const int32 LeftoverSize = Size % 2;
		Size = Size / 2;

		const int32 CheckIndex = Start + Size;
		const int32 StartIfLess = CheckIndex + LeftoverSize;

		ULandscapePatchComponent* CheckValue =  RegisteredPatches[CheckIndex].Get();
		if (!CheckValue)
		{
			bFoundInvalidPatchOut = true;
			return RegisteredPatches.Num();
		}
		Start = !PatchSortPredicateRaw(Patch, CheckValue) ? StartIfLess : Start;
	}

	return Start;
}

#endif // WITH_EDITOR
