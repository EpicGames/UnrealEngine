// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapePatchManager.h"

#include "CoreGlobals.h" // GIsReconstructingBlueprintInstances
#include "Engine/Level.h"
#include "Engine/World.h"
#include "HAL/IConsoleManager.h" // FAutoConsoleCommand
#include "Landscape.h"
#include "LandscapeEditLayerMergeRenderContext.h"
#include "LandscapeEditTypes.h"
#include "LandscapeDataAccess.h"
#include "LandscapePatchComponent.h"
#include "LandscapePatchLogging.h"
#include "LandscapePatchUtil.h"
#include "LandscapeModule.h"
#include "LandscapeEditorServices.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#include "Modules/ModuleManager.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/Package.h"
#include "UObject/UObjectBaseUtility.h" // GetNameSafe
#include "UObject/UObjectIterator.h"
#include "Algo/AnyOf.h"

#if WITH_EDITOR
#include "Editor.h"
#include "Editor/UnrealEdEngine.h"
#include "LevelEditorSubsystem.h"
#include "ScopedTransaction.h"
#include "UnrealEdGlobals.h" //GUnrealEd
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(LandscapePatchManager)

#define LOCTEXT_NAMESPACE "LandscapePatchManager"

namespace LandscapePatchManagerLocals
{
	const FText MigratePatchesTransactionName(LOCTEXT("MigratePatchesTransaction", "Migrate Patches"));

	static FAutoConsoleVariable CVarMigrateLegacyPatchListToPrioritySystem(
		TEXT("LandscapePatch.AutoMigrateLegacyListToPrioritySystem"),
		true,
		TEXT("When loaded in the editor, automatically remove all LandscapePatchManagers and bind their patches directly to an edit layer.  Set their patch priorities according to their index."));

#if WITH_EDITOR
	// Try to get a soft pointer to the owner actor for a ULandscapePatchComponent when that actor and component have not loaded.
	TSoftObjectPtr<UObject> GetActorPtrFromPatchComponent(TSoftObjectPtr<ULandscapePatchComponent> Patch)
	{
		const FSoftObjectPath& PatchPath = Patch.ToSoftObjectPath();
		if (PatchPath.IsSubobject())
		{
			FTopLevelAssetPath AssetPath = PatchPath.GetAssetPath();
			FUtf8String SubPath = PatchPath.GetSubPathUtf8String();

			// Remove the last part of the subpath, which should be the component name ".LandscapeTexturePatch", etc.
			int32 LastPathSeparator = INDEX_NONE;
			if (SubPath.FindLastChar(UTF8TEXT('.'), LastPathSeparator))
			{
				check(LastPathSeparator != INDEX_NONE);
				SubPath.LeftInline(LastPathSeparator, EAllowShrinking::No);

				FSoftObjectPath ActorPath = FSoftObjectPath::ConstructFromAssetPathAndSubpath(AssetPath, MoveTemp(SubPath));
				TSoftObjectPtr<UObject> ActorPtr(MoveTemp(ActorPath));

				return ActorPtr;
			}
		}
		return {};
	}
#endif //WITH_EDITOR
}

// TODO: Not sure if using this kind of constructor is a proper thing to do vs some other hook...
ADEPRECATED_LandscapePatchManager::ADEPRECATED_LandscapePatchManager(const FObjectInitializer& ObjectInitializer)
	: ALandscapeBlueprintBrushBase(ObjectInitializer)
{
#if WITH_EDITOR
	SetCanAffectHeightmap(true);
	SetCanAffectWeightmap(true);
	SetCanAffectVisibilityLayer(true);
#endif
}

void ADEPRECATED_LandscapePatchManager::SetTargetLandscape(ALandscape* InTargetLandscape)
{
#if WITH_EDITOR
	if (OwningLandscape != InTargetLandscape && !bDead)
	{
		if (OwningLandscape)
		{
			OwningLandscape->RemoveBrush(this);
		}

		if (!InTargetLandscape)
		{
			if (OwningLandscape != nullptr)
			{
				// This can occur if the RemoveBrush call above did not do anything because the manager
				// was removed from the landscape in some other way (probably in landscape mode panel)
				SetOwningLandscape(nullptr);
			}
			return;
		}

		if (InTargetLandscape->IsTemplate())
		{
			UE_LOGF(LogLandscapePatch, Warning, "Landscape target for patch manager is a template. Unable to attach manager.");
			SetOwningLandscape(nullptr);
		}
	}
#endif
}

#if WITH_EDITOR

void ADEPRECATED_LandscapePatchManager::MigrateToPrioritySystemAndDelete()
{
	MigrateToPrioritySystemAndDeleteInternal(/*bAllowUI=*/true);
}

void ADEPRECATED_LandscapePatchManager::MigrateToPrioritySystemAndDeleteInternal(bool bAllowUI)
{
	check(GetWorld()->WorldType == EWorldType::Editor);
	
	// Create LandscapeInfo if needed.  During load time, it might not exist yet, depending on load order.
	ULandscapeInfo* LandscapeInfo = OwningLandscape ? OwningLandscape->CreateLandscapeInfo(true) : nullptr;
	if (OwningLandscape && LandscapeInfo && !bDead && IsValid(this) && !PatchComponents.IsEmpty())
	{
		Modify();
		LandscapeInfo->MarkObjectDirty(OwningLandscape, /*bInForceResave = */true);

		// Patches will remove themselves from PatchComponents as we go along, so we need to iterate
		// a copy.
		TArray<TSoftObjectPtr<ULandscapePatchComponent>> PatchListCopy;

		// We call Modify on all the patches we'll be touching at the start, otherwise they will
		// store incorrect indices for undo as they are removed.
		int32 PatchErrors = 0;
		for (TSoftObjectPtr<ULandscapePatchComponent> Patch : PatchComponents)
		{
			if (Patch.IsPending())
			{
				Patch.LoadSynchronous();
			}

			if (!Patch.IsValid())
			{
				// Loading directly from a component TSoftObjectPtr is unreliable, so try deriving the owner actor pointer using
				// the path and load via that.  If the setup is weird and this doesn't work, it should at least still be safe. 
				TSoftObjectPtr<UObject> ActorPtr = LandscapePatchManagerLocals::GetActorPtrFromPatchComponent(Patch);
				if (ActorPtr.IsPending())
				{
					ActorPtr.LoadSynchronous();  // If this succeeds, Patch will turn valid.
				}
			}

			if (Patch.IsValid())
			{
				LandscapeInfo->MarkObjectDirty(Patch.Get(), /*bInForceResave = */true, OwningLandscape);
				Patch->Modify();
				PatchListCopy.Add(Patch);
			}
			else
			{
				// Failed to load?  This can happen if patch bindings were broken in the pre-migration scene.
				// LandscapePatch.FixPatchBindings after deleting the patch manager can fix them.
				++PatchErrors;
			}
		}

		if (!bAllowUI)
		{
			// Create the ULandscapePatchEditLayer in advance, if needed.  This prevents BindToLandscape from triggering the
			// modal UI window for choosing the new layer index.
			const ULandscapeEditLayerBase* Layer = Cast<ULandscapePatchEditLayer>(OwningLandscape->FindEditLayerOfTypeConst(ULandscapePatchEditLayer::StaticClass()));

			if (!Layer)
			{
				const FName PatchLayerName = OwningLandscape->GenerateUniqueLayerName(*ULandscapePatchEditLayer::StaticClass()->GetDefaultObject<ULandscapePatchEditLayer>()->GetDefaultName());
				// Ignore the layer count limit to avoid failing here.  It's only a soft limit to aid performance.  The old non-ULandscapePatchEditLayer is probably unused after this, but
				// it could have also been used for manual painting and we can't safely delete it.
				int32 LayerIdx = OwningLandscape->CreateLayer(PatchLayerName, ULandscapePatchEditLayer::StaticClass(), /*bIgnoreLayerCountLimit=*/ true);
				check(LayerIdx != INDEX_NONE);

				// Move the new layer beside the old-style "LandscapePatches" layer
				int32 OldPathLayerIdx = OwningLandscape->GetBrushLayer(this);
				if (ensure(OldPathLayerIdx != INDEX_NONE))
				{
					OwningLandscape->ReorderLayer(LayerIdx, OldPathLayerIdx);
				}
			}
		}

		double Priority = LEGACY_PATCH_PRIORITY_BASE;
		double PriorityStep = 1.0 / FMath::Max(1, PatchComponents.Num());

		for (TSoftObjectPtr<ULandscapePatchComponent> Patch : PatchListCopy)
		{
			Patch->SetPriority(Priority);
			Priority += PriorityStep;

			Patch->FixBindings();
		}

		UE_LOGF(LogLandscapePatch, Warning, "ALandscapePatchManager: %d landscape patches have been migrated from the legacy patch manager \"%ls\" to be bound directly to a ULandscapePatchEditLayer",
			PatchListCopy.Num(), *GetActorLabel(false));
		if (PatchErrors > 0)
		{
			UE_LOGF(LogLandscapePatch, Error, "ALandscapePatchManager: %d landscape patches failed to migrate successfully.  They may be restorable by running the command LandscapePatch.FixPatchBindings", PatchErrors);
		}

		PatchComponents.Empty();
		RequestLandscapeUpdate();
	}

	// Important so that we remove ourselves from the landscape blueprint brush list
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SetTargetLandscape(nullptr);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// bDead is used as protection from unexpected weirdness if anything happens in the window between this code running and the
	// actor being actually removed from the world (by the actionable message update button).
	bDead = true;

	if (LandscapeInfo)
	{
		// We can't delete an actor during load time, so enqueue this to be deleted later.  We want this deletion to be applied from
		// MarkModifiedLandscapesAsDirty, the same place that the deferred dirty state from MarkObjectDirty is finally applied.  This
		// will leave the scene in a consistent state before and after the user clicks the "Update" button from the landscape check
		// code.  If the scene is closed without clicking update, it will remain fully un-migrated on disk.
		LandscapeInfo->DeleteActorWhenApplyingModifiedStatus(this, bAllowUI);
	}
	else 
	{
		// No LandscapeInfo, likely because no OwningLandscape.  Try to delete directly.
		UE::Landscape::DeleteActors({ this }, GetWorld(), bAllowUI);
	}
}

void ADEPRECATED_LandscapePatchManager::PostRegisterAllComponents()
{
	Super::PostRegisterAllComponents();

	if (!IsTemplate() && IsValid(GetWorld()) && GetWorld()->WorldType == EWorldType::Editor &&
		LandscapePatchManagerLocals::CVarMigrateLegacyPatchListToPrioritySystem->GetBool())
	{
		MigrateToPrioritySystemAndDeleteInternal(/*bAllowUI=*/false);
	}
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
