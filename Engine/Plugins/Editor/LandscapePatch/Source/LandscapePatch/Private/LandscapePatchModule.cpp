// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapePatchModule.h"

#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "ShaderCore.h" // AddShaderSourceDirectoryMapping
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "FLandscapePatchModule"

void FLandscapePatchModule::StartupModule()
{
	// Maps virtual shader source directory to the plugin's actual Shaders directory.
	FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("LandscapePatch"))->GetBaseDir(), TEXT("Shaders"));
	AddShaderSourceDirectoryMapping(TEXT("/Plugin/LandscapePatch"), PluginShaderDir);
}

void FLandscapePatchModule::ShutdownModule()
{
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FLandscapePatchModule, LandscapePatch)
