// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapePatchUtil.h"

#include "LandscapeDataAccess.h" // LANDSCAPE_ZSCALE, MidValue
#include "LandscapeTexturePatchPS.h" // FSimpleTextureCopyPS
#include "RenderGraphBuilder.h"
#include "TextureResource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LandscapePatchUtil)

void UE::Landscape::PatchUtil::CopyTextureOnRenderThread(FRHICommandListImmediate& RHICmdList, 
	const FTextureResource& Source, FTextureResource& Destination)
{
	using namespace UE::Landscape;

	FRDGBuilder GraphBuilder(RHICmdList, RDG_EVENT_NAME("LandscapeTexturePatchCopyTexture"));

	FRDGTextureRef SourceTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(Source.GetTexture2DRHI(), TEXT("CopySource")));
	FRDGTextureRef DestinationTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(Destination.GetTexture2DRHI(), TEXT("CopyDestination")));

	// All my efforts of getting CopyToResolveTarget to work without complaints have failed, so we just use our own copy shader.
	FSimpleTextureCopyPS::AddToRenderGraph(GraphBuilder, SourceTexture, DestinationTexture);

	GraphBuilder.Execute();
}

FTransform UE::Landscape::PatchUtil::GetHeightmapToWorld(const FTransform& InLandscapeTransform)
{
	FTransform HeightmapCoordsToWorld;

	// Get a transform from pixel coordinate in heightmap to world space coordinate. Note that we can't
	// store the inverse directly because a FTransform can't properly represent a TRS inverse when the
	// original TRS has non-uniform scaling).

	// The pixel to landscape-space transform is unrotated, (S_p * x + T_p). The landscape to world
	// transform gets applied on top of this: (R_l * S_l * (S_p * x + T_p)) + T_L. Collapsing this
	// down to pixel to world TRS, we get: R_l * (S_l * S_p) * x + (R_l * S_l * T_p + T_L)

	// To go from stored height value to unscaled height, we divide by 128 and subtract 256. We can get these
	// values from the constants in LandscapeDataAccess.h (we distribute the multiplication by LANDSCAPE_ZSCALE
	// so that translation happens after scaling like in TRS)
	const double HEIGHTMAP_TO_OBJECT_HEIGHT_SCALE = LANDSCAPE_ZSCALE;
	const double HEIGHTMAP_TO_OBJECT_HEIGHT_OFFSET = -LandscapeDataAccess::MidValue * LANDSCAPE_ZSCALE;

	// S_p: the pixel coordinate scale is actually the same as xy object-space coordinates because one quad is 1 unit,
	// so we only need to scale the height.
	FVector3d PixelToObjectSpaceScale = FVector3d(
		1,
		1,
		HEIGHTMAP_TO_OBJECT_HEIGHT_SCALE
	);

	// T_p: the center of the pixel
	FVector3d PixelToObjectSpaceTranslate = FVector3d(
		-0.5,
		-0.5,
		HEIGHTMAP_TO_OBJECT_HEIGHT_OFFSET
	);

	// S_l* S_p: composed scale
	HeightmapCoordsToWorld.SetScale3D(InLandscapeTransform.GetScale3D() * PixelToObjectSpaceScale);

	// R_l
	HeightmapCoordsToWorld.SetRotation(InLandscapeTransform.GetRotation());

	// R_l * S_l * T_p + T_L: composed translation
	HeightmapCoordsToWorld.SetTranslation(InLandscapeTransform.TransformVector(PixelToObjectSpaceTranslate)
		+ InLandscapeTransform.GetTranslation());

	return HeightmapCoordsToWorld;
}
