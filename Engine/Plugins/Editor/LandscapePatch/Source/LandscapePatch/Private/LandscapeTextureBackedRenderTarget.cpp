// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeTextureBackedRenderTarget.h"
#include "Engine/Texture2D.h"
#include "Engine/World.h"
#include "LandscapeDataAccess.h"
#include "LandscapePatchUtil.h" // CopyTextureOnRenderThread
#include "LandscapePatchLogging.h" 
#include "LandscapeTexturePatchPS.h"
#include "RenderGraphBuilder.h"
#include "TextureCompiler.h"
#include "RenderingThread.h"
#include "TextureResource.h"
#include "UObject/ObjectSaveContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LandscapeTextureBackedRenderTarget)

namespace LandscapeTextureBackedRenderTargetLocals
{
#if WITH_EDITOR
	// Copied from CameraCalibrationUtilsPrivate.cpp
	void ClearTexture(UTexture2D* Texture, FColor ClearColor)
	{
		if (Texture)
		{
			// We need to wait for the platform data to be ready (if the texture was just created, it likely won't) :
			FTextureCompilingManager::Get().FinishCompilation({ Texture });
			if (ensure((Texture->GetPlatformData()->SizeX == Texture->GetSizeX()) && (Texture->GetPlatformData()->SizeY == Texture->GetSizeY())))
			{
			    TArray<FColor> Pixels;
			    Pixels.Init(ClearColor, Texture->GetSizeX() * Texture->GetSizeY());
    
			    void* TextureData = Texture->GetPlatformData()->Mips[0].BulkData.Lock(LOCK_READ_WRITE);
    
			    FMemory::Memcpy(TextureData, Pixels.GetData(), Pixels.Num() * sizeof(FColor));
    
			    Texture->GetPlatformData()->Mips[0].BulkData.Unlock();
			    Texture->UpdateResource();
			}
		}
	}
#endif // WITH_EDITOR
}

#if WITH_EDITOR
void ULandscapeTextureBackedRenderTargetBase::PreSave(FObjectPreSaveContext SaveContext)
{
	Super::PreSave(SaveContext);

	if (!bUseInternalTextureOnly)
	{
		CopyToInternalTexture();
	}
}

// This gets called not just when loading, but also after duplication.
void ULandscapeTextureBackedRenderTargetBase::PostLoad()
{
	Super::PostLoad();
	
	if (!bIsTemporarilyNonTransactional)
	{
		SetFlags(RF_Transactional);
		if (IsValid(InternalTexture))
		{
			InternalTexture->SetFlags(RF_Transactional);
		}
	}

	if (!bUseInternalTextureOnly && IsValid(InternalTexture))
	{
		InternalTexture->ConditionalPostLoad();

		ReinitializeRenderTarget(/*bClear*/ false);

		CopyBackFromInternalTexture();
	}

	if (IsValid(RenderTarget) && !bIsTemporarilyNonTransactional)
	{
		RenderTarget->SetFlags(RF_Transactional);
	}
}

// PreDuplicate is called not just when objects are copied in editor, but also when they are serialized for undo/redo.
void ULandscapeTextureBackedRenderTargetBase::PreDuplicate(FObjectDuplicationParameters& DupParams)
{
	Super::PreDuplicate(DupParams);

	if (!bUseInternalTextureOnly)
	{
		CopyToInternalTexture();
	}
}

// Called when serializing to text for copy/paste
void ULandscapeTextureBackedRenderTargetBase::ExportCustomProperties(FOutputDevice& Out, uint32 Indent)
{
	Super::ExportCustomProperties(Out, Indent);

	if (!bUseInternalTextureOnly)
	{
		CopyToInternalTexture();
	}
}

// Called after pasting
void ULandscapeTextureBackedRenderTargetBase::PostEditImport()
{
	Super::PostEditImport();

	if (!bUseInternalTextureOnly && IsValid(InternalTexture))
	{
		ReinitializeRenderTarget(/*bClear*/ false);

		CopyBackFromInternalTexture();
	}
}

void ULandscapeTextureBackedRenderTargetBase::PostEditUndo()
{
	Super::PostEditUndo();

	if (!bUseInternalTextureOnly && IsValid(InternalTexture))
	{
		ReinitializeRenderTarget(/*bClear*/ false);

		CopyBackFromInternalTexture();
	}
}

// This function is to support a workaround for UE-225445, where we need to temporarily mark internal objects non - transactional
//  It does so in a way that avoids them being re-marked transactional in PostLoad.
//  See comment in ULandscapeTexturePatch::GetComponentInstanceData() for details
void ULandscapeTextureBackedRenderTargetBase::SetTemporarilyNonTransactional(bool bInIsTemporarilyNonTransactional)
{
	if (bInIsTemporarilyNonTransactional)
	{
		ClearFlags(RF_Transactional);
		if (IsValid(InternalTexture))
		{
			InternalTexture->ClearFlags(RF_Transactional);
		}
		if (IsValid(RenderTarget))
		{
			RenderTarget->ClearFlags(RF_Transactional);
		}
	}
	else
	{
		SetFlags(RF_Transactional);
		if (IsValid(InternalTexture))
		{
			InternalTexture->SetFlags(RF_Transactional);
		}
		if (IsValid(RenderTarget))
		{
			RenderTarget->SetFlags(RF_Transactional);
		}
	}
	bIsTemporarilyNonTransactional = bInIsTemporarilyNonTransactional;
}

#endif // WITH_EDITOR

void ULandscapeTextureBackedRenderTargetBase::SetUseInternalTextureOnly(bool bUseInternalTextureOnlyIn, bool bCopyExisting)
{
#if WITH_EDITOR
	if (bUseInternalTextureOnly == bUseInternalTextureOnlyIn)
	{
		return;
	}

	Modify();
	bUseInternalTextureOnly = bUseInternalTextureOnlyIn;

	if (bUseInternalTextureOnly)
	{
		// We're no longer using the render target, so copy it to internal texture
		if (IsValid(RenderTarget) && bCopyExisting)
		{
			Modify();
			CopyToInternalTexture();
		}
		RenderTarget = nullptr;
	}
	else
	{
		// We're back to using the render target, so initialize it from internal texture.

		if (IsValid(InternalTexture)) // if initialized
		{
			ReinitializeRenderTarget(/*bClear*/ !bCopyExisting);

			if (bCopyExisting)
			{
				CopyBackFromInternalTexture();
			}
		}
	}
#endif // WITH_EDITOR
}

void ULandscapeTextureBackedRenderTargetBase::SetResolution(int32 SizeXIn, int32 SizeYIn)
{
#if WITH_EDITOR
	if (SizeXIn == SizeX && SizeYIn == SizeY)
	{
		return;
	}

	Modify();
	SizeX = SizeXIn;
	SizeY = SizeYIn;
	if (IsValid(RenderTarget))
	{
		RenderTarget->Modify();
		RenderTarget->InitAutoFormat(SizeX, SizeY);
		RenderTarget->UpdateResourceImmediate();
	}
	if (IsValid(InternalTexture))
	{
		InternalTexture->Modify();
		InternalTexture->Source.Init(SizeX, SizeY, 1, 1, GetInternalTextureFormat());
		InternalTexture->UpdateResource();
	}
#endif // WITH_EDITOR
}

void ULandscapeTextureBackedRenderTargetBase::Initialize()
{
#if WITH_EDITOR
	using namespace LandscapeTextureBackedRenderTargetLocals;

	if (!IsValid(InternalTexture) || !InternalTexture->GetResource())
	{
		InternalTexture = CreateInternalTexture();
	}

	InternalTexture->Source.Init(SizeX, SizeY, 1, 1, GetInternalTextureFormat());
	InternalTexture->UpdateResource();

	if (bUseInternalTextureOnly)
	{
		RenderTarget = nullptr;
	}
	else
	{
		ReinitializeRenderTarget(/*bClear*/ true);
	}
#endif // WITH_EDITOR
}



void ULandscapeWeightTextureBackedRenderTarget::SetUseAlphaChannel(bool bUseAlphaChannelIn)
{
#if WITH_EDITOR
	if (bUseAlphaChannel == bUseAlphaChannelIn)
	{
		return;
	}
	Modify();
	bUseAlphaChannel = bUseAlphaChannelIn;

	// TODO: We could try to copy the non-alpha channel data across, but that is messy with
	// an unclear amount of benefit. It would seem odd for a user to want to write one
	// way but then discard/add the alpha channel afterward. Still, could revisit later.

	if (IsValid(RenderTarget))
	{
		RenderTarget->Modify();
		RenderTarget->RenderTargetFormat = GetRenderTargetFormat();
		RenderTarget->InitAutoFormat(SizeX, SizeY);
		RenderTarget->UpdateResourceImmediate();
	}
	if (IsValid(InternalTexture))
	{
		InternalTexture->Modify();
		InternalTexture->Source.Init(SizeX, SizeY, 1, 1, GetInternalTextureFormat());
		InternalTexture->UpdateResource();
	}
#endif // WITH_EDITOR
}

ETextureRenderTargetFormat ULandscapeWeightTextureBackedRenderTarget::GetRenderTargetFormat()
{
	return bUseAlphaChannel ? ETextureRenderTargetFormat::RTF_RGBA8 : ETextureRenderTargetFormat::RTF_R8;
}

ETextureSourceFormat ULandscapeWeightTextureBackedRenderTarget::GetInternalTextureFormat()
{
	return bUseAlphaChannel ? ETextureSourceFormat::TSF_BGRA8 : ETextureSourceFormat::TSF_G8;
}

void ULandscapeWeightTextureBackedRenderTarget::Initialize()
{
#if WITH_EDITOR
	using namespace LandscapeTextureBackedRenderTargetLocals;

	Super::Initialize();

	if (bUseInternalTextureOnly && ensure(InternalTexture))
	{
		ClearTexture(InternalTexture, FColor::White);
	}
#endif // WITH_EDITOR
}

void ULandscapeWeightTextureBackedRenderTarget::CopyToInternalTexture()
{
#if WITH_EDITOR
	using namespace LandscapeTextureBackedRenderTargetLocals;

	if (!IsCopyingBackAndForthAllowed())
	{
		return;
	}

	if (!ensure(IsValid(RenderTarget)))
	{
		return;
	}

	if (!(ensure(IsValid(InternalTexture)) && InternalTexture->GetResource()))
	{
		Modify();
		InternalTexture = CreateInternalTexture();
		// The sizing and format doesn't matter because the UpdateTexture call below will deal with it.
	}
	else
	{
		// CopyToInternalTexture currently gets called in many non-dirty cases because we do not yet have a way to
		// detect a true change to the render target. So, we set bAlwaysMarkDirty here to false to avoid spuriously
		// marking the package dirty, since the internal texture may not be changing.
		InternalTexture->Modify(false);
	}

	FText ErrorMessage;
	if (RenderTarget->UpdateTexture(InternalTexture, CTF_Default, /*InAlphaOverride = */nullptr, /*InTextureChangingDelegate =*/ [](UTexture*){}, &ErrorMessage))
	{
		check(InternalTexture->Source.GetFormat() == GetInternalTextureFormat());
		InternalTexture->UpdateResource();
	}
	else
	{
		UE_LOGF(LogLandscapePatch, Error, "Couldn't copy render target to internal texture: %ls", *ErrorMessage.ToString());
	}
#endif // WITH_EDITOR
}

void ULandscapeWeightTextureBackedRenderTarget::CopyBackFromInternalTexture()
{
#if WITH_EDITOR
	using namespace LandscapeTextureBackedRenderTargetLocals;

	if (!IsCopyingBackAndForthAllowed())
	{
		return;
	}

	if (!ensure(IsValid(InternalTexture)))
	{
		return;
	}
	InternalTexture->UpdateResource();
	FTextureCompilingManager::Get().FinishCompilation({ InternalTexture });

	if (!InternalTexture->GetResource())
	{
		return;
	}

	bool bCreatedNewRenderTarget = false;
	if (!ensure(IsValid(RenderTarget)))
	{
		Modify();
		RenderTarget = NewObject<UTextureRenderTarget2D>(this);
		if (GetFlags() & RF_Transactional)
		{
			check(!bIsTemporarilyNonTransactional);
			RenderTarget->SetFlags(RF_Transactional);
		}
		bCreatedNewRenderTarget = true;
	}

	if (!ensure(InternalTexture->GetResource()->GetSizeX() == RenderTarget->SizeX
		&& InternalTexture->GetResource()->GetSizeY() == RenderTarget->SizeY
		&& RenderTarget->RenderTargetFormat == GetRenderTargetFormat()))
	{
		if (!bCreatedNewRenderTarget)
		{
			RenderTarget->Modify();
		}
		RenderTarget->RenderTargetFormat = GetRenderTargetFormat();
		RenderTarget->InitAutoFormat(InternalTexture->GetResource()->GetSizeX(), InternalTexture->GetResource()->GetSizeY());
		RenderTarget->UpdateResourceImmediate(false);
	}

	FTextureResource* Source = InternalTexture->GetResource();
	FTextureResource* Destination = RenderTarget->GetResource();

	ENQUEUE_RENDER_COMMAND(LandscapeTextureHeightPatchRTToTexture)(
		[Source, Destination](FRHICommandListImmediate& RHICmdList)
		{
			UE::Landscape::PatchUtil::CopyTextureOnRenderThread(RHICmdList, *Source, *Destination);
		});
#endif // WITH_EDITOR
}


void ULandscapeHeightTextureBackedRenderTarget::SetFormat(ETextureRenderTargetFormat FormatToUse)
{
#if WITH_EDITOR
	if (RenderTargetFormat == FormatToUse)
	{
		return;
	}
	Modify();
	RenderTargetFormat = FormatToUse;

	// We could try to copy over existing data, but that is not worth it.

	if (IsValid(RenderTarget))
	{
		RenderTarget->Modify();
		RenderTarget->RenderTargetFormat = GetRenderTargetFormat();
		RenderTarget->InitAutoFormat(SizeX, SizeY);
		RenderTarget->UpdateResourceImmediate();
	}
#endif // WITH_EDITOR
}

void ULandscapeHeightTextureBackedRenderTarget::Initialize()
{
#if WITH_EDITOR
	using namespace LandscapeTextureBackedRenderTargetLocals;
	Super::Initialize();

	const FColor LandscapeNativeMidHeightColor = LandscapeDataAccess::PackHeight(LandscapeDataAccess::MidValue);

	if (bUseInternalTextureOnly && ensure(InternalTexture))
	{
		ClearTexture(InternalTexture, LandscapeNativeMidHeightColor);
	}
#endif // WITH_EDITOR
}

void ULandscapeHeightTextureBackedRenderTarget::CopyToInternalTexture()
{
#if WITH_EDITOR
	using namespace LandscapeTextureBackedRenderTargetLocals;

	if (!IsCopyingBackAndForthAllowed())
	{
		return;
	}

	if (!ensure(IsValid(RenderTarget)))
	{
		return;
	}

	if (!(ensure(IsValid(InternalTexture)) && InternalTexture->GetResource()))
	{
		Modify();
		InternalTexture = CreateInternalTexture();
	}
	else
	{
		// CopyToInternalTexture currently gets called in many non-dirty cases because we do not yet have a way to
		// detect a true change to the render target. So, we set bAlwaysMarkDirty here to false to avoid spuriously
		// marking the package dirty, since the internal texture may not be changing.
		InternalTexture->Modify(false);
	}

	UTextureRenderTarget2D* NativeEncodingRenderTarget = RenderTarget;

	// If the format doesn't match the format that we use generally for our internal texture, save the patch in our native
	// height format, applying whatever scale/offset is relevant. The stored texture thus ends up being the native equivalent
	// (with scale 1 and offset 0). This is easier than trying to support various kinds of RT-to-texture conversions.
	if (NativeEncodingRenderTarget->RenderTargetFormat != ETextureRenderTargetFormat::RTF_RGBA8)
	{
		// We need a temporary render target to write the converted result, then we'll copy that to the texture.
		NativeEncodingRenderTarget = NewObject<UTextureRenderTarget2D>(this);
		NativeEncodingRenderTarget->RenderTargetFormat = ETextureRenderTargetFormat::RTF_RGBA8;
		NativeEncodingRenderTarget->InitAutoFormat(RenderTarget->SizeX, RenderTarget->SizeY);
		NativeEncodingRenderTarget->UpdateResourceImmediate(false);

		FTextureResource* Source = RenderTarget->GetResource();
		FTextureResource* Destination = NativeEncodingRenderTarget->GetResource();

		ENQUEUE_RENDER_COMMAND(LandscapeTextureHeightPatchRTToTexture)(
			[this, Source, Destination](FRHICommandListImmediate& RHICmdList)
			{
				using namespace UE::Landscape;

				FRDGBuilder GraphBuilder(RHICmdList, RDG_EVENT_NAME("LandscapeTextureHeightPatchConvertToNative"));

				FRDGTextureRef SourceTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(Source->GetTexture2DRHI(), TEXT("ConversionSource")));
				FRDGTextureRef DestinationTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(Destination->GetTexture2DRHI(), TEXT("ConversionDestination")));

				FConvertToNativeLandscapePatchPS::AddToRenderGraph(GraphBuilder, SourceTexture, DestinationTexture, ConversionParams);

				GraphBuilder.Execute();
			});
	}

	// This call does a flush for us, so the render target should be updated.
	FText ErrorMessage;
	if (NativeEncodingRenderTarget->UpdateTexture(InternalTexture, CTF_Default, /*InAlphaOverride = */nullptr, /*InTextureChangingDelegate =*/ [](UTexture*) {}, &ErrorMessage))
	{
		check(InternalTexture->Source.GetFormat() == GetInternalTextureFormat());
		InternalTexture->UpdateResource();
	}
	else
	{
		UE_LOGF(LogLandscapePatch, Error, "Couldn't copy render target to internal texture: %ls", *ErrorMessage.ToString());
	}
#endif // WITH_EDITOR
}

void ULandscapeHeightTextureBackedRenderTarget::CopyBackFromInternalTexture()
{
#if WITH_EDITOR
	using namespace LandscapeTextureBackedRenderTargetLocals;

	if (!IsCopyingBackAndForthAllowed())
	{
		return;
	}

	if (!ensure(IsValid(InternalTexture)))
	{
		return;
	}

	InternalTexture->UpdateResource();
	FTextureCompilingManager::Get().FinishCompilation({ InternalTexture });

	if (!InternalTexture->GetResource())
	{
		return;
	}

	bool bCreatedNewRenderTarget = false;
	if (!ensure(IsValid(RenderTarget)))
	{
		Modify();
		RenderTarget = NewObject<UTextureRenderTarget2D>(this);
		if (GetFlags() & RF_Transactional)
		{
			check(!bIsTemporarilyNonTransactional);
			RenderTarget->SetFlags(RF_Transactional);
		}
		bCreatedNewRenderTarget = true;
	}

	if (!ensure(InternalTexture->GetResource()->GetSizeX() == RenderTarget->SizeX
		&& InternalTexture->GetResource()->GetSizeY() == RenderTarget->SizeY
		&& RenderTarget->RenderTargetFormat == GetRenderTargetFormat()))
	{
		if (!bCreatedNewRenderTarget)
		{
			RenderTarget->Modify();
		}
		RenderTarget->RenderTargetFormat = GetRenderTargetFormat();
		RenderTarget->InitAutoFormat(InternalTexture->GetResource()->GetSizeX(), InternalTexture->GetResource()->GetSizeY());
	}

	FTextureResource* Source = InternalTexture->GetResource();
	FTextureResource* Destination = RenderTarget->GetResource();
	if (!ensure(Source && Destination))
	{
		return;
	}

	// If we're in a different format, we need to "un-bake" the height from the texture.
	if (RenderTarget->RenderTargetFormat != ETextureRenderTargetFormat::RTF_RGBA8)
	{
		ENQUEUE_RENDER_COMMAND(LandscapeTextureHeightPatchRTToTexture)(
			[this, Source, Destination](FRHICommandListImmediate& RHICmdList)
			{
				using namespace UE::Landscape;

				FRDGBuilder GraphBuilder(RHICmdList, RDG_EVENT_NAME("LandscapeTextureHeightPatchConvertFromNative"));

				FRDGTextureRef SourceTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(Source->GetTexture2DRHI(), TEXT("ConversionSource")));
				FRDGTextureRef DestinationTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(Destination->GetTexture2DRHI(), TEXT("ConversionDestination")));

				FConvertBackFromNativeLandscapePatchPS::AddToRenderGraph(GraphBuilder, SourceTexture, DestinationTexture, ConversionParams);

				GraphBuilder.Execute();
			});
	}
	else
	{
		// When formats match, we can just copy back and forth.
		ENQUEUE_RENDER_COMMAND(LandscapeTextureHeightPatchRTToTexture)(
			[Source, Destination](FRHICommandListImmediate& RHICmdList)
			{
				UE::Landscape::PatchUtil::CopyTextureOnRenderThread(RHICmdList, *Source, *Destination);
			});
	}
#endif // WITH_EDITOR
}

bool ULandscapeTextureBackedRenderTargetBase::IsCopyingBackAndForthAllowed()
{
	UWorld* World = GetWorld();
	return IsValidChecked(this) && !IsTemplate()
		// Note that having a null world is ok because we get that temporarily while rerunning construction
		// scripts. However if we do have a world, it should be the normal editor one.
		&& (!World || (IsValid(World) && World->WorldType == EWorldType::Editor))
		&& FApp::CanEverRender();
}

void ULandscapeTextureBackedRenderTargetBase::ReinitializeRenderTarget(bool bClear)
{
	if (!IsValid(RenderTarget))
	{
		Modify();
		RenderTarget = NewObject<UTextureRenderTarget2D>(this);
		if (GetFlags() & RF_Transactional)
		{
			check(!bIsTemporarilyNonTransactional);
			RenderTarget->SetFlags(RF_Transactional);
		}
	}
	else
	{
		RenderTarget->Modify();
	}
	RenderTarget->RenderTargetFormat = GetRenderTargetFormat();
	RenderTarget->InitAutoFormat(SizeX, SizeY);
	RenderTarget->UpdateResourceImmediate(bClear);
}

#if WITH_EDITOR
UTexture2D* ULandscapeTextureBackedRenderTargetBase::CreateInternalTexture()
{
	UTexture2D* Texture = NewObject<UTexture2D>(this);
	if (GetFlags() & RF_Transactional)
	{
		check(!bIsTemporarilyNonTransactional);
		Texture->SetFlags(RF_Transactional);
	}
	Texture->SRGB = false;
	Texture->MipGenSettings = TMGS_NoMipmaps;
	Texture->AddressX = TA_Clamp;
	Texture->AddressY = TA_Clamp;
	// TODO: How do we allow lossless compression, instead of disallowing compression entirely? Just setting 
	// LossyCompressionAmount to ETextureLossyCompressionAmount::TLCA_None is not sufficient.
	Texture->CompressionNone = true;
	// If we don't do this, then each newly created empty texture created will request the DDC to re-compile it,
	// which is problematic if the texture is being recreated over and over in a construction script.
	Texture->Source.UseHashAsGuid();

	return Texture;
}
#endif // WITH_EDITOR