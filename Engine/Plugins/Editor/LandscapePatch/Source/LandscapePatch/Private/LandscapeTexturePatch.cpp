// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeTexturePatch.h"

#include "Engine/TextureRenderTarget2DArray.h"
#include "Engine/World.h"
#include "Landscape.h"
#include "LandscapeDataAccess.h"
#include "LandscapeEditLayerMergeRenderContext.h"
#include "LandscapeEditLayerTargetTypeState.h"
#include "LandscapeEditResourcesSubsystem.h" // ULandscapeScratchRenderTarget
#include "LandscapePatchLogging.h"
#include "LandscapePatchManager.h"
#include "LandscapePatchUtil.h" // CopyTextureOnRenderThread
#include "LandscapeTexturePatchPS.h"
#include "LandscapeUtils.h" // IsVisibilityLayer
#include "Logging/MessageLog.h"
#include "MathUtil.h"
#include "Misc/UObjectToken.h"
#include "RHIStaticStates.h"
#include "RenderGraphUtils.h"
#include "RenderingThread.h"
#include "SystemTextures.h"
#include "TextureResource.h"
#include "Algo/AnyOf.h"
#include "Algo/ForEach.h"
#include "Containers/Ticker.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LandscapeTexturePatch)

#define LOCTEXT_NAMESPACE "LandscapeTexturePatch"

namespace LandscapeTexturePatchLocals
{
#if WITH_EDITOR
	template <typename TextureBackedRTType>
	void TransitionSourceMode(ELandscapeTexturePatchSourceMode OldMode, ELandscapeTexturePatchSourceMode NewMode,
		TObjectPtr<UTexture>& OutTextureAsset, TObjectPtr<TextureBackedRTType>& OutInternalData, ELandscapeTexturePatchTextureChannel& OutTextureChannel,
		TUniqueFunction<TextureBackedRTType* ()> InternalDataBuilder)
	{
		// When transitioning from an invalid source mode enable the red channel (default color channel used by patch)
		if (OutTextureChannel == ELandscapeTexturePatchTextureChannel::None && NewMode != OldMode)
		{
			OutTextureChannel = ELandscapeTexturePatchTextureChannel::Red;
		}

		if (NewMode == ELandscapeTexturePatchSourceMode::None)
		{
			OutTextureAsset = nullptr;
			OutInternalData = nullptr;

			// Reset the color channel
			OutTextureChannel = ELandscapeTexturePatchTextureChannel::None;
		}
		else if (NewMode == ELandscapeTexturePatchSourceMode::TextureAsset)
		{
			OutInternalData = nullptr;
		}
		else // new mode is internal texture or render target
		{
			bool bWillUseTextureOnly = (NewMode == ELandscapeTexturePatchSourceMode::InternalTexture);
			bool bNeedToCopyTextureAsset = (OldMode == ELandscapeTexturePatchSourceMode::TextureAsset
				&& IsValid(OutTextureAsset) && OutTextureAsset->GetResource());

			if (!OutInternalData)
			{
				OutInternalData = InternalDataBuilder();
				OutInternalData->SetUseInternalTextureOnly(bWillUseTextureOnly && !bNeedToCopyTextureAsset);
				OutInternalData->Initialize();
			}
			else
			{
				OutInternalData->Modify();
			}

			OutInternalData->SetUseInternalTextureOnly(bWillUseTextureOnly && !bNeedToCopyTextureAsset);
			if (bNeedToCopyTextureAsset)
			{
				// Copy the currently set texture asset to our render target
				FTextureResource* Source = OutTextureAsset->GetResource();
				FTextureResource* Destination = OutInternalData->GetRenderTarget()->GetResource();

				ENQUEUE_RENDER_COMMAND(LandscapeTextureHeightPatchRTToTexture)(
					[Source, Destination](FRHICommandListImmediate& RHICmdList)
					{
						UE::Landscape::PatchUtil::CopyTextureOnRenderThread(RHICmdList, *Source, *Destination);
					});
			}

			// Note that the duplicate SetUseInternalTextureOnly calls (in cases where we don't need to copy the texture asset)
			// are fine because they don't do anything.
			OutInternalData->SetUseInternalTextureOnly(bWillUseTextureOnly);

			OutTextureAsset = nullptr;
		}
	}

	// TODO: The way we currently do initialization is a bit of a hack in that we actually request to do
	//  a landscape update but we read instead of writing. In batched merge, this might not always work
	//  properly because a patch might be at the edge of a rendered batch, and thus only have part of it
	//  be initialized properly. The proper way to do reinitialization would be to use a special function
	//  to render the relevant part of the landscape directly to the patch. We should do this at some point,
	//  but it is not high priority because reinitialization does not currently seem to be commonly used.
	// @param PatchToHeightmapUVs This is expected to be a usual math matrix by this point, not Unreal's transposed one
	void DoReinitializationOverlapCheck(FMatrix44f& PatchToHeightmapUVs, int32 PatchTextureSizeX, int32 PatchTextureSizeY)
	{
		auto IsInsideHeightmap = [&PatchToHeightmapUVs](int32 X, int32 Y)->bool
		{
			float U = PatchToHeightmapUVs.M[0][0] * X + PatchToHeightmapUVs.M[0][1] * Y + PatchToHeightmapUVs.M[0][3];
			float V = PatchToHeightmapUVs.M[1][0] * X + PatchToHeightmapUVs.M[1][1] * Y + PatchToHeightmapUVs.M[1][3];

			return U >= 0 && U <= 1 && V >= 0 && V <= 1;
		};

		if (!IsInsideHeightmap(0, 0)
			|| !IsInsideHeightmap(0, PatchTextureSizeY-1)
			|| !IsInsideHeightmap(PatchTextureSizeX-1, 0)
			|| !IsInsideHeightmap(PatchTextureSizeX-1, PatchTextureSizeY-1))
		{
			UE_LOGF(LogLandscapePatch, Warning, "ULandscapeTexturePatch::Reinitialize: Part or all of the patch was outside "
				"a region of landscape being rendered. Reinitialization might not work be fully supported here.");
		}
	}

	void GetTexturePatchAlphaResource(FLandscapeTexturePatchAlphaSettings& InAlphaSettings, UObject* InOuter, FTextureResource*& OutAlphaTextureResource)
	{
		UTexture* PatchAlphaUObject = nullptr;

		switch (InAlphaSettings.GetAlphaSourceMode())
		{
		case ELandscapeTexturePatchAlphaSourceMode::None:
			break;
		case ELandscapeTexturePatchAlphaSourceMode::SourceTextureChannel:
			break;
		case ELandscapeTexturePatchAlphaSourceMode::InternalTexture:
			PatchAlphaUObject = InAlphaSettings.GetAlphaInternalTexture(InOuter);
			break;
		case ELandscapeTexturePatchAlphaSourceMode::TextureBackedRenderTarget:
			PatchAlphaUObject = InAlphaSettings.GetAlphaRenderTarget(InOuter);
			break;
		case ELandscapeTexturePatchAlphaSourceMode::TextureAsset:
			// The alpha texture can be invalid but cannot be a virtual texture
			if (InAlphaSettings.AlphaTextureAsset && InAlphaSettings.AlphaTextureAsset->VirtualTextureStreaming != 0)
			{
				UE_LOGF(LogLandscapePatch, Error, "ULandscapeTexturePatch::GetTexturePatchAlphaResource: Virtual textures are not supported. ");
				return;
			}
			PatchAlphaUObject = InAlphaSettings.AlphaTextureAsset;
			break;
		default:
			checkNoEntry();
		}

		if (IsValid(PatchAlphaUObject))
		{
			OutAlphaTextureResource = PatchAlphaUObject->GetResource();
		}
	}
#endif // WITH_EDITOR

	bool bUseExternalTextureFix = true;
	FAutoConsoleVariableRef CVarUseExternalTextureAlignmentFix(
		TEXT("LandscapePatch.UseExternalTextureAlignmentFix"), bUseExternalTextureFix, 
		TEXT("Can be used to roll back an external texture alignment fix if temporarily needed."));

	FVector2D GetCoverageWithExtraPixel(const FVector2D& Resolution, const FVector2D& Coverage)
	{
		// UnscaledPatchCoverage is meant to represent the distance between the centers of the extremal pixels.
		//  That distance in pixels is Resolution-1.
		FVector2D TargetPixelSize(Coverage / FVector2D::Max(Resolution - 1, FVector2D(1, 1)));
		return TargetPixelSize * Resolution;
	}

	// Alpha source mode implements all default source mode options plus an additional SourceTextureChannel
	// The SourceTextureChannel mode does not allocate any assets/internal data, so it's commonly equivalent to ELandscapeTexturePatchSourceMode::None
	inline ELandscapeTexturePatchSourceMode AlphaModeToSourceMode(ELandscapeTexturePatchAlphaSourceMode InMode)
	{
		switch (InMode)
		{
		// Unlike the other alpha modes, SourceTextureChannel does not add an additional standalone texture
		case ELandscapeTexturePatchAlphaSourceMode::None:
		case ELandscapeTexturePatchAlphaSourceMode::SourceTextureChannel:
			return ELandscapeTexturePatchSourceMode::None;
		case ELandscapeTexturePatchAlphaSourceMode::InternalTexture:
			return ELandscapeTexturePatchSourceMode::InternalTexture;
		case ELandscapeTexturePatchAlphaSourceMode::TextureBackedRenderTarget:
			return ELandscapeTexturePatchSourceMode::TextureBackedRenderTarget;
		case ELandscapeTexturePatchAlphaSourceMode::TextureAsset:
			return ELandscapeTexturePatchSourceMode::TextureAsset;
		default:
			checkNoEntry();
		}
		return ELandscapeTexturePatchSourceMode::None;
	}
}

FMatrix44d FLandscapeTexturePatchTextureTransform::GetTransformAsMatrix(const FTransform& PatchToWorldIn, const FVector2f& PatchWorldDimensionsIn) const
{
	const FVector PatchWorldScale = PatchToWorldIn.GetScale3D();
	const FVector2f ScaledPatchDimensions = PatchWorldDimensionsIn * FVector2f(PatchWorldScale.X, PatchWorldScale.Y);

	// Clamp scale to avoid divide by zero
	const double ScaleX = FMath::Abs(Scale.X) < SMALL_NUMBER ? FMath::FloatSelect(Scale.X, 1.0, -1.0) * SMALL_NUMBER : Scale.X;
	const double ScaleY = FMath::Abs(Scale.Y) < SMALL_NUMBER ? FMath::FloatSelect(Scale.Y, 1.0, -1.0) * SMALL_NUMBER : Scale.Y;

	// Offset may be in [0, 1] range or in world units
	FVector UVOffset = FVector(Offset.X, Offset.Y, 0.0);

	// Both world/relative sampling support local and world scaling factors
	if (bUseWorldPositionSampling)
	{
		// Return WorldtoTexture matrix
		const FVector UVWorldScale = FVector(ScaleX, ScaleY, 1.0);
		const FVector UVRelativeScale = FVector(ScaledPatchDimensions.X * ScaleX, ScaledPatchDimensions.Y * ScaleY, 1.0);

		// Add half the texture UV tiling scale to center the texture around origin. Scale will be in [0, 1] range or world units
		UVOffset += FVector(ScaleX / 2, ScaleY / 2, 0);

		// By default the transform is in world units. When using relative scale, multiply by the patch dimensions to get the offset in relative UV space
		if (!bUseWorldSpaceScale)
		{
			UVOffset *= FVector(ScaledPatchDimensions.X, ScaledPatchDimensions.Y, 0);
		}

		const FTransform TextureUVToWorld = FTransform(FRotator(0, Rotation, 0), UVOffset, bUseWorldSpaceScale ? UVWorldScale : UVRelativeScale);
		// Return a WorldToTextureUV matrix
		return TextureUVToWorld.ToInverseMatrixWithScale();
	}
	else
	{
		// This matrix is used to apply a rotation, offset, scale to the patch's UVs
		FVector UVRelativeScale = FVector(1.0 / ScaleX, 1.0 / ScaleY, 1.0);
		FVector2f UVWorldScale = ScaledPatchDimensions / FVector2f(ScaleX, ScaleY);

		return FTransform(FRotator(0.0, Rotation, 0.0), UVOffset, bUseWorldSpaceScale ? FVector(UVWorldScale.X, UVWorldScale.Y, 1.0) : UVRelativeScale).ToMatrixWithScale();
	}
}

UTextureRenderTarget2D* FLandscapeTexturePatchAlphaSettings::GetAlphaRenderTarget(UObject* InOuter, bool bAlwaysMarkDirty/*=true*/)
{
#if WITH_EDITOR
	// InOuter is the texture patch or weight patch layer info
	if (!InOuter || InOuter->IsTemplate())
	{
		return nullptr;
	}

	// Allocate data if needed (see comment in GetHeightRenderTarget)
	if (AlphaSourceMode == ELandscapeTexturePatchAlphaSourceMode::TextureBackedRenderTarget)
	{
		if (!InternalAlphaData || !InternalAlphaData->GetRenderTarget())
		{
			SetAlphaSourceMode(AlphaSourceMode, InOuter, bAlwaysMarkDirty);
		}

		return ensure(InternalAlphaData) ? InternalAlphaData->GetRenderTarget() : nullptr;
	}
#endif

	return nullptr;
}

UTexture2D* FLandscapeTexturePatchAlphaSettings::GetAlphaInternalTexture(UObject* InOuter, bool bAlwaysMarkDirty/*=true*/)
{
#if WITH_EDITOR
	// InOuter is the texture patch or weight patch layer info
	if (!InOuter || InOuter->IsTemplate())
	{
		return nullptr;
	}

	// Allocate data if needed (see comment in GetHeightRenderTarget)
	if (AlphaSourceMode == ELandscapeTexturePatchAlphaSourceMode::InternalTexture
		|| AlphaSourceMode == ELandscapeTexturePatchAlphaSourceMode::TextureBackedRenderTarget)
	{
		if (!InternalAlphaData || !InternalAlphaData->GetInternalTexture())
		{
			SetAlphaSourceMode(AlphaSourceMode, InOuter, bAlwaysMarkDirty);
		}

		return ensure(InternalAlphaData) ? InternalAlphaData->GetInternalTexture() : nullptr;
	}
#endif

	return nullptr;
}

void FLandscapeTexturePatchAlphaSettings::SetAlphaSourceMode(ELandscapeTexturePatchAlphaSourceMode InNewSourceMode, UObject* InOuter, bool bAlwaysMarkDirty/*=true*/)
{
#if WITH_EDITOR
	using namespace LandscapeTexturePatchLocals;
	if (InNewSourceMode == AlphaSourceMode)
	{
		return;
	}

	// InOuter is the texture patch or weight patch layer info
	if (!InOuter->IsTemplate())
	{
		InOuter->Modify(bAlwaysMarkDirty);

		FVector2D Resolution;
		// If a weight patch is passed in get the outer texture patch
		if (ULandscapeTexturePatch* WeightTexturePatch = Cast<ULandscapeTexturePatch>(InOuter->GetOuter()))
		{
			Resolution = WeightTexturePatch->GetResolution();
		}
		else
		{
			// The owner is the patch (used by default weight alpha settings / heightmap alpha settings)
			ULandscapeTexturePatch* TexturePatch = CastChecked<ULandscapeTexturePatch>(InOuter);
			Resolution = TexturePatch->GetResolution();
		}

		// Convert from alpha mode to source mode for transition
		ELandscapeTexturePatchSourceMode OldMode = AlphaModeToSourceMode(AlphaSourceMode);
		ELandscapeTexturePatchSourceMode NewMode = AlphaModeToSourceMode(InNewSourceMode);

		TransitionSourceMode<ULandscapeWeightTextureBackedRenderTarget>(OldMode, NewMode, AlphaTextureAsset, InternalAlphaData, AlphaTextureChannel, [&Resolution, this, InOuter]()
		{
			check(InOuter);
			ULandscapeWeightTextureBackedRenderTarget* InternalDataToReturn = NewObject<ULandscapeWeightTextureBackedRenderTarget>(InOuter);
			// Only set the transactional flag on that new sub-object if the outer is (it can happen that the outer is temporarily made non-transactional in which case, we don't want
			//  one of its sub-objects to become transactional) : 
			if (InOuter->GetFlags() & RF_Transactional)
			{
				InternalDataToReturn->SetFlags(RF_Transactional);
			}
			InternalDataToReturn->SetResolution(Resolution.X, Resolution.Y);
			// Alpha texture only uses a single color channel and will use RTF_R8/TSF_G8 formats for internal texture/render targets
			InternalDataToReturn->SetUseAlphaChannel(false);
			return InternalDataToReturn;
		});

		// TransitionSourceMode doesn't handle alpha specific mode SourceTextureChannel. 
		// When the user has no texture channel selected, automatically enable the alpha channel (default color channel in this mode) 
		if (AlphaTextureChannel == ELandscapeTexturePatchTextureChannel::None && InNewSourceMode == ELandscapeTexturePatchAlphaSourceMode::SourceTextureChannel)
		{
			AlphaTextureChannel = ELandscapeTexturePatchTextureChannel::Alpha;
		}
	}
	// In a template, it is not safe to try to allocate a texture, etc. All we do is clear out the
	// texture asset pointer if it is not needed, to avoid referencing assets unnecessarily.
	else if (AlphaSourceMode != ELandscapeTexturePatchAlphaSourceMode::TextureAsset)
	{
		AlphaTextureAsset = nullptr;
	}

	AlphaSourceMode = InNewSourceMode;
	DetailPanelAlphaSourceMode = InNewSourceMode;
#endif // WITH_EDITOR
}

#if WITH_EDITOR
UE::Landscape::EditLayers::ERenderFlags ULandscapeTexturePatch::GetRenderFlags(const UE::Landscape::EditLayers::FMergeContext* InMergeContext) const
{
	using namespace UE::Landscape::EditLayers;

	ERenderFlags RenderFlags = ERenderFlags::None;
	// COMMENT [jonathan.bard] : this is not something we want to keep (we will use partial edit layer renders for this eventually) but we can still render in immediate mode
	//  in the "reinitialize on next render case" because we perform a synchronous read then so we need to run on the game thread to perform the rendering commands flush 
	if (InMergeContext->IsHeightmapMerge() && bReinitializeHeightOnNextRender)
	{
		RenderFlags |= ERenderFlags::RenderMode_Immediate;
	}
	else if (!InMergeContext->IsHeightmapMerge() && Algo::AnyOf(WeightPatches, [](const TObjectPtr<ULandscapeWeightPatchTextureInfo>& InWeightPatch)
		{
			return IsValid(InWeightPatch) && InWeightPatch->bReinitializeOnNextRender;
		}))
	{
		RenderFlags |= ERenderFlags::RenderMode_Immediate;
	}
	else
	{
		RenderFlags |= ERenderFlags::RenderMode_Recorded;
	}
	RenderFlags |= ERenderFlags::BlendMode_SeparateBlend | ERenderFlags::RenderLayerGroup_SupportsGrouping;
	return RenderFlags;
}

bool ULandscapeTexturePatch::CanGroupRenderLayerWith(TScriptInterface<ILandscapeEditLayerRenderer> InOtherRenderer) const
{
	UObject* OtherRenderer = InOtherRenderer.GetObject();
	check(OtherRenderer != nullptr);
	// Texture patches are compatible with one another (blend mode is handled per-pixel): 
	return OtherRenderer->IsA<ULandscapeTexturePatch>();
}

bool ULandscapeTexturePatch::RenderLayer(UE::Landscape::EditLayers::FRenderParams& RenderParams, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder)
{
	using namespace UE::Landscape;
	using namespace UE::Landscape::PatchUtil;
	using namespace UE::Landscape::EditLayers;

	FTransform LandscapeHeightmapToWorld = GetHeightmapToWorld(RenderParams.RenderAreaWorldTransform);

	ULandscapeScratchRenderTarget* LandscapeScratchRT = RenderParams.MergeRenderContext->GetBlendRenderTargetWrite();

	const bool bIsHeightmapTarget = RenderParams.MergeRenderContext->IsHeightmapMerge();
	if (bIsHeightmapTarget)
	{
		UTextureRenderTarget2D* CurrentData = LandscapeScratchRT->TryGetRenderTarget2D();
		if (!ensure(CurrentData))
		{
			return false;
		}

		if (bReinitializeHeightOnNextRender)
		{
			bReinitializeHeightOnNextRender = false;
			checkf(!RDGBuilderRecorder.IsRecording(), TEXT("We should be using ERenderFlags::RenderMode_Immediate when reinitializing height"));
			ReinitializeHeight(CurrentData, LandscapeHeightmapToWorld);
			return true;
		}
		else
		{
			checkf(RDGBuilderRecorder.IsRecording(), TEXT("We should be using ERenderFlags::RenderMode_Recorded in the typical case"));
			bool bHasRenderedSomething = false;
			ApplyToHeightmap(&RenderParams, RDGBuilderRecorder, CurrentData, GetHeightmapToWorld(RenderParams.RenderAreaWorldTransform), bHasRenderedSomething);
			return bHasRenderedSomething;
		}
	}

	// If we got to here, we're dealing with weightmaps.

	UTextureRenderTarget2DArray* TextureArray = LandscapeScratchRT->TryGetRenderTarget2DArray();
	check(TextureArray != nullptr);
	
	int32 NumTargetLayersInGroup = RenderParams.TargetLayerGroupLayerNames.Num();
	check(LandscapeScratchRT->GetEffectiveNumSlices() == NumTargetLayersInGroup);

	bool bHasRenderedSomething = false;

	for (int32 TargetLayerIndexInGroup = 0; TargetLayerIndexInGroup < NumTargetLayersInGroup; ++TargetLayerIndexInGroup)
	{
		bool bIsVisibilityLayer = ensure(TargetLayerIndexInGroup < RenderParams.TargetLayerGroupLayerInfos.Num())
			&& UE::Landscape::IsVisibilityLayer(RenderParams.TargetLayerGroupLayerInfos[TargetLayerIndexInGroup]);
		
		// Try to find the weight patch
		ULandscapeWeightPatchTextureInfo* FoundWeightPatch = nullptr;
		for (const TObjectPtr<ULandscapeWeightPatchTextureInfo>& WeightPatch : WeightPatches)
		{
			if (IsValid(WeightPatch))
			{
				if ((bIsVisibilityLayer && WeightPatch->bEditVisibilityLayer)
					|| (WeightPatch->WeightmapLayerName == RenderParams.TargetLayerGroupLayerNames[TargetLayerIndexInGroup]))
				{
					FoundWeightPatch = WeightPatch;
					break;
				}
			}
		}

		if (!FoundWeightPatch)
		{
			// Didn't have a patch for this weight layer
			continue;
		}

		if (FoundWeightPatch->bReinitializeOnNextRender)
		{
			FoundWeightPatch->bReinitializeOnNextRender = false;
			checkf(!RDGBuilderRecorder.IsRecording(), TEXT("We should be using ERenderFlags::RenderMode_Immediate when reinitializing weight"));
			ReinitializeWeightPatch(FoundWeightPatch, TextureArray->GetResource(),
				FIntPoint(TextureArray->SizeX, TextureArray->SizeY), TargetLayerIndexInGroup, LandscapeHeightmapToWorld);

			bHasRenderedSomething = true;
		}
		else
		{
			checkf(RDGBuilderRecorder.IsRecording(), TEXT("We should be using ERenderFlags::RenderMode_Recorded in the typical case"));
			ApplyToWeightmap(&RenderParams, RDGBuilderRecorder, FoundWeightPatch,
				TextureArray->GetResource(),
				TargetLayerIndexInGroup,
				RenderParams.RenderAreaSectionRect.Size(), 
				GetHeightmapToWorld(RenderParams.RenderAreaWorldTransform), 
				bHasRenderedSomething);
		}
	}//end for each layer index

	return bHasRenderedSomething;
}//end ULandscapeTexturePatch::RenderLayer

void ULandscapeTexturePatch::BlendLayer(UE::Landscape::EditLayers::FRenderParams& RenderParams, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder)
{
	using namespace UE::Landscape::EditLayers;

	// Prepare the generic blend params based on the patch's data : 
	FBlendParams BlendParams;
	if (RenderParams.MergeRenderContext->IsHeightmapMerge())
	{
		BlendParams.HeightmapBlendParams.BlendMode = EHeightmapBlendMode::AlphaBlend;
	}
	else 
	{
		BlendParams.WeightmapBlendParams.Reserve(RenderParams.TargetLayerGroupLayerNames.Num());
		for (FName TargetLayerName : RenderParams.TargetLayerGroupLayerNames)
		{
			// only blend the layers involved in this step (the others are using EWeightmapBlendMode::Passthrough): 
			FWeightmapBlendParams& TargetLayerBlendParams = BlendParams.WeightmapBlendParams.Emplace(TargetLayerName, EWeightmapBlendMode::AlphaBlend);
		}
	}

	// Then perform the generic blend : 
	RenderParams.MergeRenderContext->GenericBlendLayer(BlendParams, RenderParams, RDGBuilderRecorder);
}

UTextureRenderTarget2D* ULandscapeTexturePatch::ApplyToHeightmap(UE::Landscape::EditLayers::FRenderParams* RenderParams, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder, UTextureRenderTarget2D* InCombinedResult,
	const FTransform& LandscapeHeightmapToWorld, bool& bHasRenderedSomething, ERHIAccess OutputAccess)
{
	using namespace UE::Landscape;

	// Get the source of our height patch
	UTexture* PatchUObject = nullptr;
	switch (HeightSourceMode)
	{
	case ELandscapeTexturePatchSourceMode::None:
		return InCombinedResult;
	case ELandscapeTexturePatchSourceMode::InternalTexture:
		PatchUObject = GetHeightInternalTexture();
		break;
	case ELandscapeTexturePatchSourceMode::TextureBackedRenderTarget:
		PatchUObject = GetHeightRenderTarget(/*bMarkDirty = */ false);
		break;
	case ELandscapeTexturePatchSourceMode::TextureAsset:
		if (IsValid(HeightTextureAsset) && HeightTextureAsset->VirtualTextureStreaming != 0)
		{
			UE_LOGF(LogLandscapePatch, Error, "ULandscapeTexturePatch::ApplyToHeightmap: Invalid texture. Virtual textures are not supported. ");
			return InCombinedResult;
		}
		PatchUObject = HeightTextureAsset;
		break;
	default:
		checkNoEntry();
	}

	if (!IsValid(PatchUObject))
	{
		return InCombinedResult;
	}

	FTextureResource* PatchTextureResource = PatchUObject->GetResource();
	if (!PatchTextureResource)
	{
		return InCombinedResult;
	}

	// Get the heightmap texture alpha override resource
	FTextureResource* PatchAlphaTextureResource = nullptr;
	LandscapeTexturePatchLocals::GetTexturePatchAlphaResource(HeightAlphaSettings, this, PatchAlphaTextureResource);

	// Go ahead and pack everything into a copy of the param struct so we don't have to capture everything
	// individually in the lambda below.
	FApplyLandscapeTextureHeightPatchPSParameters ShaderParamsToCopy;
	FIntRect DestinationBounds;
	GetHeightShaderParams(LandscapeHeightmapToWorld, FIntPoint(PatchTextureResource->GetSizeX(), PatchTextureResource->GetSizeY()), FIntPoint(InCombinedResult->SizeX, InCombinedResult->SizeY), ShaderParamsToCopy, DestinationBounds);

	if (DestinationBounds.Area() <= 0)
	{
		// Patch must be outside the landscape.
		return InCombinedResult;
	}

	FTextureResource* OutputResource = InCombinedResult->GetResource();
	FString OutputResourceName = OutputResource->GetResourceName().ToString();
		check(RenderParams != nullptr);
		ULandscapeScratchRenderTarget* WriteRT = RenderParams->MergeRenderContext->GetBlendRenderTargetWrite();
		// After this point, the render cannot fail so if we're the first in our render layer group to render, we can cycle the blend render targets and 
		//  start rendering in the write one : 
		if (!bHasRenderedSomething && (RenderParams->NumSuccessfulRenderLayerStepsUntilBlendLayerStep == 0))
		{
			RenderParams->MergeRenderContext->CycleBlendRenderTargets(RDGBuilderRecorder);
			WriteRT = RenderParams->MergeRenderContext->GetBlendRenderTargetWrite();
			WriteRT->Clear(RDGBuilderRecorder);
			check(WriteRT->GetCurrentState() == ERHIAccess::RTV);
		}
		OutputResource = WriteRT->GetRenderTarget()->GetResource();
		OutputResourceName = WriteRT->GetDebugName();

	auto RDGCommand =
		[ OutputResource
		, OutputResourceName
		, ShaderParamsToCopy
		, PatchTextureResource
		, PatchAlphaTextureResource
		, DestinationBounds](FRDGBuilder& GraphBuilder)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(LandscapeTextureHeightPatch_Render);

			TRefCountPtr<IPooledRenderTarget> DestinationRenderTarget = CreateRenderTarget(OutputResource->GetTexture2DRHI(), TEXT("LandscapeTextureHeightPatchOutput"));
			FRDGTextureRef DestinationTexture = GraphBuilder.RegisterExternalTexture(DestinationRenderTarget);

			FApplyLandscapeTextureHeightPatchPS::FParameters* ShaderParams =
				GraphBuilder.AllocParameters<FApplyLandscapeTextureHeightPatchPS::FParameters>();
			*ShaderParams = ShaderParamsToCopy;

			TRefCountPtr<IPooledRenderTarget> PatchRenderTarget = CreateRenderTarget(PatchTextureResource->GetTexture2DRHI(), TEXT("LandscapeTextureHeightPatch"));
			FRDGTextureRef PatchTexture = GraphBuilder.RegisterExternalTexture(PatchRenderTarget);
			FRDGTextureSRVRef PatchSRV = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(PatchTexture, 0));
			ShaderParams->InHeightPatch = PatchSRV;
			ShaderParams->InHeightPatchSampler = TStaticSamplerState<SF_Bilinear, AM_Wrap, AM_Wrap, AM_Wrap>::GetRHI();

			// Heightmap alpha override resource. Pass white system texture when invalid (no transparency)
			if (PatchAlphaTextureResource != nullptr && PatchAlphaTextureResource->GetTexture2DRHI())
			{
				TRefCountPtr<IPooledRenderTarget> PatchAlphaRenderTarget = CreateRenderTarget(PatchAlphaTextureResource->GetTexture2DRHI(), TEXT("LandscapeAlphaTextureHeightPatch"));
				FRDGTextureRef PatchAlphaTexture = GraphBuilder.RegisterExternalTexture(PatchAlphaRenderTarget);
				FRDGTextureSRVRef PatchAlphaSRV = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(PatchAlphaTexture, 0));
				ShaderParams->InHeightPatchAlphaTexture = PatchAlphaSRV;
			}
			else
			{
				ShaderParams->InHeightPatchAlphaTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(GSystemTextures.GetWhiteDummy(GraphBuilder)));
			}

			ShaderParams->RenderTargets[0] = FRenderTargetBinding(DestinationTexture, ERenderTargetLoadAction::ENoAction, /*InMipIndex = */0);

			FApplyLandscapeTextureHeightPatchPS::AddToRenderGraph(RDG_EVENT_NAME("RenderTextureHeightPatch -> %s", *OutputResourceName), GraphBuilder, ShaderParams, DestinationBounds);
		};
	// We need to specify the final state of the external texture to prevent the graph builder from transitioning it to SRVMask :
	RDGBuilderRecorder.EnqueueRDGCommand(RDGCommand, { { OutputResource, (OutputAccess == ERHIAccess::None) ? ERHIAccess::RTV : OutputAccess} });

	bHasRenderedSomething = true;
	return InCombinedResult;
}

void ULandscapeTexturePatch::ApplyToWeightmap(UE::Landscape::EditLayers::FRenderParams* RenderParams, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder, ULandscapeWeightPatchTextureInfo* PatchInfo,
	FTextureResource* InMergedLandscapeTextureResource, int32 LandscapeTextureSliceIndex,
	const FIntPoint& LandscapeTextureResolution, const FTransform& LandscapeHeightmapToWorld, bool& bHasRenderedSomething, ERHIAccess OutputAccess)
{
	using namespace UE::Landscape;

	if (!PatchInfo)
	{
		return;
	}

	UTexture* PatchUObject = nullptr;

	switch (PatchInfo->SourceMode)
	{
	case ELandscapeTexturePatchSourceMode::None:
		return;
	case ELandscapeTexturePatchSourceMode::InternalTexture:
		PatchUObject = GetWeightPatchInternalTexture(PatchInfo);
		break;
	case ELandscapeTexturePatchSourceMode::TextureBackedRenderTarget:
		PatchUObject = GetWeightPatchRenderTarget(PatchInfo);
		break;
	case ELandscapeTexturePatchSourceMode::TextureAsset:
		if (IsValid(PatchInfo->TextureAsset) && PatchInfo->TextureAsset->VirtualTextureStreaming != 0)
		{
			UE_LOGF(LogLandscapePatch, Error, "ULandscapeTexturePatch::ApplyToWeightmap: Invalid texture. Virtual textures are not supported. ");
			return;
		}
		PatchUObject = PatchInfo->TextureAsset;
		break;
	default:
		checkNoEntry();
	}

	if (!IsValid(PatchUObject))
	{
		return;
	}

	FTextureResource* PatchTextureResource = PatchUObject->GetResource();
	if (!PatchTextureResource)
	{
		return;
	}

	// Get the alpha override or default weight patches alpha settings
	FTextureResource* PatchAlphaTextureResource = nullptr;
	FLandscapeTexturePatchAlphaSettings& InAlphaSettings = GetWeightPatchAlphaSettings(PatchInfo);
	LandscapeTexturePatchLocals::GetTexturePatchAlphaResource(InAlphaSettings, PatchInfo, PatchAlphaTextureResource);

	// Go ahead and pack everything into a copy of the param struct so we don't have to capture everything
	// individually in the lambda below.
	FApplyLandscapeTextureWeightPatchPSParameters ShaderParamsToCopy;
	FIntRect DestinationBounds;

	GetWeightShaderParams(LandscapeHeightmapToWorld, FIntPoint(PatchTextureResource->GetSizeX(), PatchTextureResource->GetSizeY()),
		LandscapeTextureResolution, PatchInfo, ShaderParamsToCopy, DestinationBounds);

	if (DestinationBounds.Area() <= 0)
	{
		// Patch must be outside the landscape.
		return;
	}

	FTextureResource* OutputResource = InMergedLandscapeTextureResource;
	FString OutputResourceName = OutputResource->GetResourceName().ToString();
		check(RenderParams != nullptr);
		ULandscapeScratchRenderTarget* WriteRT = RenderParams->MergeRenderContext->GetBlendRenderTargetWrite();
		// After this point, the render cannot fail so if we're the first in our render layer group to render, we can cycle the blend render targets and 
		//  start rendering in the write one : 
		if (!bHasRenderedSomething && (RenderParams->NumSuccessfulRenderLayerStepsUntilBlendLayerStep == 0))
		{
			RenderParams->MergeRenderContext->CycleBlendRenderTargets(RDGBuilderRecorder);
			WriteRT = RenderParams->MergeRenderContext->GetBlendRenderTargetWrite();
			WriteRT->Clear(RDGBuilderRecorder);
			check(WriteRT->GetCurrentState() == ERHIAccess::RTV);
		}
		OutputResource = WriteRT->GetRenderTarget()->GetResource();
		OutputResourceName = WriteRT->GetDebugName();

	auto RDGCommand =
		[ OutputResource
		, OutputResourceName
		, LandscapeTextureSliceIndex
		, ShaderParamsToCopy
		, PatchTextureResource
		, PatchAlphaTextureResource
		, DestinationBounds](FRDGBuilder& GraphBuilder)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(LandscapeTextureWeightPatch_Render);

			TRefCountPtr<IPooledRenderTarget> DestinationRenderTarget = CreateRenderTarget(OutputResource->GetTextureRHI(), TEXT("LandscapeTextureWeightPatchOutput"));
			FRDGTextureRef DestinationTexture = GraphBuilder.RegisterExternalTexture(DestinationRenderTarget);

			FApplyLandscapeTextureWeightPatchPS::FParameters* ShaderParams =
				GraphBuilder.AllocParameters<FApplyLandscapeTextureWeightPatchPS::FParameters>();
			*ShaderParams = ShaderParamsToCopy;

			TRefCountPtr<IPooledRenderTarget> PatchRenderTarget = CreateRenderTarget(PatchTextureResource->GetTexture2DRHI(), TEXT("LandscapeTextureWeightPatch"));
			FRDGTextureRef PatchTexture = GraphBuilder.RegisterExternalTexture(PatchRenderTarget);
			FRDGTextureSRVRef PatchSRV = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(PatchTexture, 0));
			ShaderParams->InWeightPatch = PatchSRV;
			ShaderParams->InWeightPatchSampler = TStaticSamplerState<SF_Bilinear, AM_Wrap, AM_Wrap, AM_Wrap>::GetRHI();

			// Weightmap alpha override resource. Pass white system texture when invalid (no transparency)
			if (PatchAlphaTextureResource != nullptr && PatchAlphaTextureResource->GetTexture2DRHI())
			{
				TRefCountPtr<IPooledRenderTarget> PatchAlphaRenderTarget = CreateRenderTarget(PatchAlphaTextureResource->GetTexture2DRHI(), TEXT("LandscapeAlphaTextureWeightPatch"));
				FRDGTextureRef PatchAlphaTexture = GraphBuilder.RegisterExternalTexture(PatchAlphaRenderTarget);
				FRDGTextureSRVRef PatchAlphaSRV = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(PatchAlphaTexture, 0));
				ShaderParams->InWeightPatchAlphaTexture = PatchAlphaSRV;
			}
			else
			{
				ShaderParams->InWeightPatchAlphaTexture = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::Create(GSystemTextures.GetWhiteDummy(GraphBuilder)));
			}
			
			ShaderParams->RenderTargets[0] = FRenderTargetBinding(DestinationTexture, ERenderTargetLoadAction::ENoAction,
				/*InMipIndex = */0, LandscapeTextureSliceIndex);

			FApplyLandscapeTextureWeightPatchPS::AddToRenderGraph(RDG_EVENT_NAME("RenderTextureWeightPatch -> %s", *OutputResourceName), GraphBuilder, ShaderParams, DestinationBounds);
		};

	// We need to specify the final state of the external texture to prevent the graph builder from transitioning it to SRVMask :
	RDGBuilderRecorder.EnqueueRDGCommand(RDGCommand, { { InMergedLandscapeTextureResource, (OutputAccess == ERHIAccess::None) ? ERHIAccess::RTV : OutputAccess} });
	
	bHasRenderedSomething = true;
}

void ULandscapeTexturePatch::GetCommonShaderParams(const FTransform& LandscapeHeightmapToWorldIn,
	const FIntPoint& SourceResolutionIn, const FIntPoint& DestinationResolutionIn,
	const FLandscapeTexturePatchTextureTransform& SourceTextureTransformIn, const FLandscapeTexturePatchTextureTransform& AlphaTextureTransformIn,
	FTransform& PatchToWorldOut, FVector2f& PatchWorldDimensionsOut, FMatrix44f& HeightmapToPatchUVOut, FMatrix44f& HeightmapToPatchUVTransformOut,
	FMatrix44f& HeightmapAlphaToPatchUVTransformOut, FIntRect& DestinationBoundsOut, FVector2f& EdgeUVDeadBorderOut, float& FalloffWorldMarginOut) const
{
	using namespace LandscapeTexturePatchLocals;

	PatchToWorldOut = GetPatchToWorldTransform();

	FVector2D FullPatchDimensions = bUseExternalTextureFix ? GetCoverageWithExtraPixel(FVector2D(SourceResolutionIn), GetUnscaledCoverage())
		: GetFullUnscaledWorldSize();
	PatchWorldDimensionsOut = FVector2f(FullPatchDimensions);

	FTransform FromPatchUVToPatch(FQuat4d::Identity, FVector3d(-FullPatchDimensions.X / 2, -FullPatchDimensions.Y / 2, 0),
		FVector3d(FullPatchDimensions.X, FullPatchDimensions.Y, 1));
	FMatrix44d PatchLocalToUVs = FromPatchUVToPatch.ToInverseMatrixWithScale();

	FMatrix44d LandscapeToWorld = LandscapeHeightmapToWorldIn.ToMatrixWithScale();

	FMatrix44d WorldToPatch = PatchToWorldOut.ToInverseMatrixWithScale();

	// In unreal, matrix composition is done by multiplying the subsequent ones on the right, and the result
	// is transpose of what our shader will expect (because unreal right multiplies vectors by matrices).
	FMatrix44d LandscapeToPatchUVTransposed = LandscapeToWorld * WorldToPatch * PatchLocalToUVs;
	HeightmapToPatchUVOut = static_cast<FMatrix44f>(LandscapeToPatchUVTransposed.GetTransposed());

	// The shader uses HeightmapToPatchUV matrix to map SVPos (batched merge texture space) to the [0, 1] UV space. This is useful for applying falloff along the patch edges
	// The transformed matrices apply the source texture's transform (Scale, Offset, Rotation) to use custom texture sampling within the patch bounds (UV tiling)
	if (SourceTextureTransformIn.bUseWorldPositionSampling)
	{
		// World space matrix transform : LandscapeHeightmapToWorldIn * WorldToTexture
		const FMatrix44d WorldToTextureUV = SourceTextureTransformIn.GetTransformAsMatrix(PatchToWorldOut, PatchWorldDimensionsOut);
		const FMatrix44d HeightmapToTextureUV = LandscapeToWorld * WorldToTextureUV;
		HeightmapToPatchUVTransformOut = static_cast<FMatrix44f>(HeightmapToTextureUV.GetTransposed());
	}
	else
	{
		// Relative space matrix transform : LandscapeToPatchUVTransposed is the default PatchUV space (based on patch's bounds) 
		const FMatrix44d LandscapeToTransformPatchUV = LandscapeToPatchUVTransposed * SourceTextureTransformIn.GetTransformAsMatrix(PatchToWorldOut, PatchWorldDimensionsOut);
		HeightmapToPatchUVTransformOut = static_cast<FMatrix44f>(LandscapeToTransformPatchUV.GetTransposed());
	}

	// Build the alpha texture transformed matrix
	if (AlphaTextureTransformIn.bUseWorldPositionSampling)
	{
		// World space matrix transform : LandscapeHeightmapToWorldIn * WorldToTexture
		const FMatrix44d WorldToAlphaTextureUV = AlphaTextureTransformIn.GetTransformAsMatrix(PatchToWorldOut, PatchWorldDimensionsOut);
		const FMatrix44d HeightmapAlphaToTextureUV = LandscapeToWorld * WorldToAlphaTextureUV;
		HeightmapAlphaToPatchUVTransformOut = static_cast<FMatrix44f>(HeightmapAlphaToTextureUV.GetTransposed());
	}
	else
	{
		// Relative space matrix transform : LandscapeToPatchUVTransposed is the default PatchUV space (based on patch's bounds) 
		const FMatrix44d LandscapeToTransformPatchAlphaUV = LandscapeToPatchUVTransposed * AlphaTextureTransformIn.GetTransformAsMatrix(PatchToWorldOut, PatchWorldDimensionsOut);
		HeightmapAlphaToPatchUVTransformOut = static_cast<FMatrix44f>(LandscapeToTransformPatchAlphaUV.GetTransposed());
	}

	// Get the output bounds, which are used to limit the amount of landscape pixels we have to process. 
	// To get them, convert all of the corners into heightmap 2d coordinates and get the bounding box.
	auto PatchUVToHeightmap2DCoordinates = [&PatchToWorldOut, &FromPatchUVToPatch, &LandscapeHeightmapToWorldIn](const FVector2f& UV)
	{
		FVector WorldPosition = PatchToWorldOut.TransformPosition(
			FromPatchUVToPatch.TransformPosition(FVector(UV.X, UV.Y, 0)));
		FVector HeightmapCoordinates = LandscapeHeightmapToWorldIn.InverseTransformPosition(WorldPosition);
		return FVector2d(HeightmapCoordinates.X, HeightmapCoordinates.Y);
	};
	FBox2D FloatBounds(ForceInit);
	FloatBounds += PatchUVToHeightmap2DCoordinates(FVector2f(0, 0));
	FloatBounds += PatchUVToHeightmap2DCoordinates(FVector2f(0, 1));
	FloatBounds += PatchUVToHeightmap2DCoordinates(FVector2f(1, 0));
	FloatBounds += PatchUVToHeightmap2DCoordinates(FVector2f(1, 1));

	DestinationBoundsOut = FIntRect(
		FMath::Clamp(FMath::Floor(FloatBounds.Min.X), 0, DestinationResolutionIn.X - 1),
		FMath::Clamp(FMath::Floor(FloatBounds.Min.Y), 0, DestinationResolutionIn.Y - 1),
		FMath::Clamp(FMath::CeilToInt(FloatBounds.Max.X) + 1, 0, DestinationResolutionIn.X),
		FMath::Clamp(FMath::CeilToInt(FloatBounds.Max.Y) + 1, 0, DestinationResolutionIn.Y));

	// The outer half-pixel shouldn't affect the landscape because it is not part of our official coverage area.
	EdgeUVDeadBorderOut = FVector2f::Zero();
	if (SourceResolutionIn.X * SourceResolutionIn.Y != 0)
	{
		EdgeUVDeadBorderOut = FVector2f(0.5 / SourceResolutionIn.X, 0.5 / SourceResolutionIn.Y);
	}

	FVector3d ComponentScale = PatchToWorldOut.GetScale3D();
	FalloffWorldMarginOut = Falloff / FMath::Min(ComponentScale.X, ComponentScale.Y);
}

void ULandscapeTexturePatch::GetHeightShaderParams(const FTransform& LandscapeHeightmapToWorldIn,
	const FIntPoint& SourceResolutionIn, const FIntPoint& DestinationResolutionIn,
	UE::Landscape::FApplyLandscapeTextureHeightPatchPSParameters& ParamsOut,
	FIntRect& DestinationBoundsOut) const
{
	using namespace UE::Landscape;

	FTransform PatchToWorld;
	GetCommonShaderParams(LandscapeHeightmapToWorldIn, SourceResolutionIn, DestinationResolutionIn, HeightTextureUVTransform, HeightAlphaSettings.AlphaTextureUVTransform,
		PatchToWorld, ParamsOut.InPatchWorldDimensions, ParamsOut.InHeightmapToPatchUV, ParamsOut.InHeightmapToPatchUVTransform, ParamsOut.InHeightmapAlphaToPatchUVTransform, 
		DestinationBoundsOut, ParamsOut.InEdgeUVDeadBorder, ParamsOut.InFalloffWorldMargin);

	FVector3d ComponentScale = PatchToWorld.GetScale3D();
	double LandscapeHeightScale = Landscape.IsValid() ? Landscape->GetTransform().GetScale3D().Z : 1;
	LandscapeHeightScale = LandscapeHeightScale == 0 ? 1 : LandscapeHeightScale;

	bool bNativeEncoding = HeightSourceMode == ELandscapeTexturePatchSourceMode::InternalTexture
		|| HeightEncoding == ELandscapeTextureHeightPatchEncoding::NativePackedHeight;

	// To get height scale in heightmap coordinates, we have to undo the scaling that happens to map the 16bit int to [-256, 256), and undo
	// the landscape actor scale.
	ParamsOut.InHeightScale = bNativeEncoding ? 1
		: LANDSCAPE_INV_ZSCALE * HeightEncodingSettings.WorldSpaceEncodingScale / LandscapeHeightScale;
	if (bApplyComponentZScale)
	{
		ParamsOut.InHeightScale *= ComponentScale.Z;
	}

	ParamsOut.InZeroInEncoding = bNativeEncoding ? LandscapeDataAccess::MidValue : HeightEncodingSettings.ZeroInEncoding;

	ParamsOut.InHeightOffset = 0;
	switch (ZeroHeightMeaning)
	{
	case ELandscapeTextureHeightPatchZeroHeightMeaning::LandscapeZ:
		break; // no offset necessary
	case ELandscapeTextureHeightPatchZeroHeightMeaning::PatchZ:
	{
		FVector3d PatchOriginInHeightmapCoords = LandscapeHeightmapToWorldIn.InverseTransformPosition(PatchToWorld.GetTranslation());
		ParamsOut.InHeightOffset = PatchOriginInHeightmapCoords.Z - LandscapeDataAccess::MidValue;
		break;
	}
	case ELandscapeTextureHeightPatchZeroHeightMeaning::WorldZero:
	{
		FVector3d WorldOriginInHeightmapCoords = LandscapeHeightmapToWorldIn.InverseTransformPosition(FVector::ZeroVector);
		ParamsOut.InHeightOffset = WorldOriginInHeightmapCoords.Z - LandscapeDataAccess::MidValue;
		break;
	}
	default:
		checkNoEntry();
	}

	ParamsOut.InBlendMode = static_cast<uint32>(BlendMode);

	// Pack our booleans into a bitfield
	using EShaderFlags = FApplyLandscapeTextureHeightPatchPS::EFlags;
	EShaderFlags Flags = EShaderFlags::None;

	Flags |= (FalloffMode == ELandscapeTexturePatchFalloffMode::RoundedRectangle) ?
		EShaderFlags::RectangularFalloff : EShaderFlags::None;

	Flags |= bNativeEncoding ?
		EShaderFlags::InputIsPackedHeight : EShaderFlags::None;

	ParamsOut.InFlags = static_cast<uint8>(Flags);
	check(!bNativeEncoding || HeightTextureChannel == ELandscapeTexturePatchTextureChannel::None);
	ParamsOut.InTextureChannel = static_cast<uint8>(HeightTextureChannel);

	ParamsOut.InAlphaMode = static_cast<uint8>(HeightAlphaSettings.GetAlphaSourceMode());
	ParamsOut.InAlphaTextureChannel = static_cast<uint8>(HeightAlphaSettings.AlphaTextureChannel);

	const ULandscapePatchEditLayer* PatchEditLayer = GetBoundEditLayer();
	check(PatchEditLayer != nullptr);
	ParamsOut.InEditLayerHeightmapAlpha = PatchEditLayer->GetAlphaForTargetType(ELandscapeToolTargetType::Heightmap);
}

void ULandscapeTexturePatch::GetWeightShaderParams(
	const FTransform& LandscapeHeightmapToWorldIn, const FIntPoint& SourceResolutionIn,
	const FIntPoint& DestinationResolutionIn, const ULandscapeWeightPatchTextureInfo* WeightPatchInfo, 
	UE::Landscape::FApplyLandscapeTextureWeightPatchPSParameters& ParamsOut,
	FIntRect& DestinationBoundsOut) const
{
	using namespace UE::Landscape;

	FTransform PatchToWorld;
	const FLandscapeTexturePatchTextureTransform& AlphaTextureTransform = GetWeightPatchAlphaTextureTransform(WeightPatchInfo->WeightmapLayerName);

	GetCommonShaderParams(LandscapeHeightmapToWorldIn, SourceResolutionIn, DestinationResolutionIn, WeightPatchInfo->WeightTextureUVTransform, AlphaTextureTransform,
		PatchToWorld, ParamsOut.InPatchWorldDimensions, ParamsOut.InWeightmapToPatchUV, ParamsOut.InWeightmapToPatchUVTransform, ParamsOut.InWeightmapAlphaToPatchUVTransform,
		DestinationBoundsOut, ParamsOut.InEdgeUVDeadBorder, ParamsOut.InFalloffWorldMargin);

	// Use the override blend mode if present, otherwise fall back to more general blend mode.
	ParamsOut.InBlendMode = static_cast<uint32>(WeightPatchInfo->bOverrideBlendMode ? WeightPatchInfo->OverrideBlendMode : WeightPatchesBlendMode);

	// Pack our booleans into a bitfield
	using EShaderFlags = FApplyLandscapeTextureHeightPatchPS::EFlags;
	EShaderFlags Flags = EShaderFlags::None;

	Flags |= (FalloffMode == ELandscapeTexturePatchFalloffMode::RoundedRectangle) ?
		EShaderFlags::RectangularFalloff : EShaderFlags::None;

	ParamsOut.InFlags = static_cast<uint8>(Flags);
	ParamsOut.InTextureChannel = static_cast<uint8>(WeightPatchInfo->WeightPatchTextureChannel);

	ParamsOut.InAlphaMode = static_cast<uint8>(GetWeightPatchAlphaSourceMode(WeightPatchInfo->WeightmapLayerName));
	ParamsOut.InAlphaTextureChannel = static_cast<uint8>(GetWeightPatchAlphaTextureChannel(WeightPatchInfo->WeightmapLayerName));

	const ULandscapePatchEditLayer* PatchEditLayer = GetBoundEditLayer();
	check(PatchEditLayer != nullptr);
	ParamsOut.InEditLayerWeightmapAlpha = PatchEditLayer->GetAlphaForTargetType(ELandscapeToolTargetType::Weightmap);
}

// This function determines how our internal height render targets get converted to the format that gets
// serialized. In a perfect world, this largely shouldn't matter as long as we don't lose data in the conversion
// back and forth. In practice, it matters for transitioning the SourceMode between ELandscapeTexturePatchSourceMode::InternalTexture 
// and ELandscapeTexturePatchSourceMode::TextureBackedRenderTarget, and it matters for reinitializing the patch
// from the current landscape. In the former, it matters because the transition is easy if the backing format
// is the same as the equivalent texture. In the latter, it matters because the reinitialization is easy if
// the backing format is the same as the applied landscape values. Currently we end up making the former easy, i.e.
// we serialize render targets to their equivalent native texture representation, and don't bake in the offset.
// This means that we need to do a bit more work when reinitializing to account for the offset.
// It should also be noted that there are some truncation/rounding implications to the choices made here that
// only matter if the user is messing around with the conversion parameters and hoping not to lose data... But
// there's a limited amount that we can protect the user in that case anyway.
FLandscapeHeightPatchConvertToNativeParams ULandscapeTexturePatch::GetHeightConvertToNativeParams() const
{
	// When doing conversions, we bake into a height in the same way that we do when applying the patch.

	FLandscapeHeightPatchConvertToNativeParams ConversionParams;
	ConversionParams.ZeroInEncoding = HeightEncodingSettings.ZeroInEncoding;

	double LandscapeHeightScale = Landscape.IsValid() ? Landscape->GetTransform().GetScale3D().Z : 1;
	LandscapeHeightScale = LandscapeHeightScale == 0 ? 1 : LandscapeHeightScale;
	ConversionParams.HeightScale = HeightEncodingSettings.WorldSpaceEncodingScale * LANDSCAPE_INV_ZSCALE / LandscapeHeightScale;

	// See above discussion about why we don't currently bake in height offset.
	ConversionParams.HeightOffset = 0;

	return ConversionParams;
}

#endif // WITH_EDITOR

void ULandscapeTexturePatch::RequestReinitializeHeight()
{
#if WITH_EDITOR
	if (!Super::IsEnabled())
	{
		UE_LOGF(LogLandscapePatch, Warning, "ULandscapeTexturePatch::ReinitializeHeight: Cannot reinitialize while disabled.");
		return;
	}

	if (!Landscape.IsValid())
	{
		UE_LOGF(LogLandscapePatch, Warning, "ULandscapeTexturePatch::ReinitializeHeight: No associated landscape to initialize from.");
		return;
	}

	if (!GetBoundEditLayer())
	{
		UE_LOGF(LogLandscapePatch, Warning, "ULandscapeTexturePatch::ReinitializeHeight: Not bound to landscape (via edit layer).");
		return;
	}

	FVector2D DesiredResolution(FMath::Max(1, InitTextureSizeX), FMath::Max(1, InitTextureSizeY));
	if (bBaseResolutionOffLandscape)
	{
		GetInitResolutionFromLandscape(ResolutionMultiplier, DesiredResolution);
	}
	SetResolution(DesiredResolution);

	bReinitializeHeightOnNextRender = true;
	RequestLandscapeUpdate();

#endif // WITH_EDITOR
}

void ULandscapeTexturePatch::RequestReinitializeWeights()
{
#if WITH_EDITOR
	if (!Super::IsEnabled())
	{
		UE_LOGF(LogLandscapePatch, Warning, "ULandscapeTexturePatch::ReinitializeWeights: Cannot reinitialize while disabled.");
		return;
	}

	if (!Landscape.IsValid())
	{
		UE_LOGF(LogLandscapePatch, Warning, "ULandscapeTexturePatch::ReinitializeWeights: No associated landscape to initialize from.");
		return;
	}

	if (!GetBoundEditLayer())
	{
		UE_LOGF(LogLandscapePatch, Warning, "ULandscapeTexturePatch::ReinitializeWeights: Not bound to landscape (via edit layer).");
		return;
	}

	FVector2D DesiredResolution(FMath::Max(1, InitTextureSizeX), FMath::Max(1, InitTextureSizeY));
	if (bBaseResolutionOffLandscape)
	{
		GetInitResolutionFromLandscape(ResolutionMultiplier, DesiredResolution);
	}
	SetResolution(DesiredResolution);

	ULandscapeInfo* Info = Landscape->GetLandscapeInfo();
	if (Info)
	{
		for (const FLandscapeInfoLayerSettings& InfoLayerSettings : Info->Layers)
		{
			if (!InfoLayerSettings.LayerInfoObj)
			{
				continue;
			}
			
			FName WeightmapLayerName = InfoLayerSettings.GetLayerName();
			bool bIsVisibilityLayer = UE::Landscape::IsVisibilityLayer(InfoLayerSettings.LayerInfoObj);

			// Minor note: there's some undefined behavior if a user uses a patch that both has bEditVisibilityLayer
			//  set to true and a weight layer name that matches some other weight layer. That's ok.
			TArray<TObjectPtr<ULandscapeWeightPatchTextureInfo>> FoundPatches;
			if (bIsVisibilityLayer)
			{
				FoundPatches = WeightPatches.FilterByPredicate([](const TObjectPtr<ULandscapeWeightPatchTextureInfo>& InWeightPatch)
				{
					return IsValid(InWeightPatch) && InWeightPatch->bEditVisibilityLayer;
				});
			}
			else
			{
				if (!ensure(WeightmapLayerName != NAME_None))
				{
					continue;
				}
				FoundPatches = WeightPatches.FilterByPredicate(
					[&WeightmapLayerName](const TObjectPtr<ULandscapeWeightPatchTextureInfo>& InWeightPatch)
				{ 
					return IsValid(InWeightPatch) && InWeightPatch->WeightmapLayerName == WeightmapLayerName;
				});
			}

			if (FoundPatches.IsEmpty())
			{
				CreateWeightPatch(WeightmapLayerName, ELandscapeTexturePatchSourceMode::InternalTexture, ELandscapeTexturePatchAlphaSourceMode::None);
				WeightPatches.Last()->bReinitializeOnNextRender = true;
				WeightPatches.Last()->bEditVisibilityLayer = bIsVisibilityLayer;
			}
			else
			{
				for (const TObjectPtr<ULandscapeWeightPatchTextureInfo>& WeightPatch : FoundPatches)
				{
					if (IsValid(WeightPatch))
					{
						WeightPatch->bReinitializeOnNextRender = true;
					}
				}
			}
		}
		RequestLandscapeUpdate();
	}

#endif // WITH_EDITOR
}

#if WITH_EDITOR
void ULandscapeTexturePatch::ReinitializeHeight(UTextureRenderTarget2D* InCombinedResult, const FTransform& LandscapeHeightmapToWorld)
{
	using namespace LandscapeTexturePatchLocals;

	if (HeightSourceMode == ELandscapeTexturePatchSourceMode::TextureAsset)
	{
		UE_LOGF(LogLandscapePatch, Warning, "ULandscapeTexturePatch: Cannot reinitialize height patch when source mode is an external texture.");
		return;
	}

	if (HeightSourceMode == ELandscapeTexturePatchSourceMode::None)
	{
		SetHeightSourceMode(ELandscapeTexturePatchSourceMode::InternalTexture);
	}
	else if (IsValid(HeightInternalData))
	{
		if (HeightSourceMode == ELandscapeTexturePatchSourceMode::InternalTexture && IsValid(HeightInternalData->GetInternalTexture()))
		{
			HeightInternalData->GetInternalTexture()->Modify();
		}
		else if (HeightSourceMode == ELandscapeTexturePatchSourceMode::TextureBackedRenderTarget && IsValid(HeightInternalData->GetRenderTarget()))
		{
			HeightInternalData->GetRenderTarget()->Modify();
		}
	}

	if (!ensure(IsValid(HeightInternalData)))
	{
		return;
	}

	SetHeightAlphaSourceMode(ELandscapeTexturePatchAlphaSourceMode::None);
	SetBlendMode(ELandscapeTexturePatchBlendMode::AlphaBlend);
	ResetHeightRenderTargetFormat();

	// The way we're going to do it is that we'll copy the packed values directly to a temporary render target, offset 
	// them if needed (to undo whatever offsetting will happen during application), and store the result directly in the
	// backing internal texture. Then we'll update the actual associated render target from the internal texture (if needed) so
	// that unpacking and height format conversion happens the same way as everywhere else.

	// We do need to make sure that the scale conversion for the backing texture matches what will be used when applying it.
	UpdateHeightConvertToNativeParamsIfNeeded();

	UTextureRenderTarget2D* TemporaryNativeHeightCopy = NewObject<UTextureRenderTarget2D>(this);
	TemporaryNativeHeightCopy->RenderTargetFormat = ETextureRenderTargetFormat::RTF_RGBA8;
	TemporaryNativeHeightCopy->InitAutoFormat(ResolutionX, ResolutionY);
	TemporaryNativeHeightCopy->UpdateResourceImmediate(true);
	
	// If ZeroHeightMeaning is not landscape Z, then we're going to be applying an offset to our data when
	// applying it to landscape, which means we'll need to apply the inverse offset when initializing here
	// so that we get the same landscape back.
	double OffsetToApply = 0;
	if (ZeroHeightMeaning != ELandscapeTextureHeightPatchZeroHeightMeaning::LandscapeZ)
	{
		double ZeroHeight = 0;
		if (ZeroHeightMeaning == ELandscapeTextureHeightPatchZeroHeightMeaning::PatchZ)
		{
			ZeroHeight = LandscapeHeightmapToWorld.InverseTransformPosition(GetComponentTransform().GetTranslation()).Z;
		}
		else if (ZeroHeightMeaning == ELandscapeTextureHeightPatchZeroHeightMeaning::WorldZero)
		{
			ZeroHeight = LandscapeHeightmapToWorld.InverseTransformPosition(FVector::ZeroVector).Z;
		}
		OffsetToApply = LandscapeDataAccess::MidValue - ZeroHeight;
	}

	FMatrix44f PatchToSource = GetPatchToHeightmapUVs(LandscapeHeightmapToWorld, TemporaryNativeHeightCopy->SizeX, TemporaryNativeHeightCopy->SizeY, InCombinedResult->SizeX, InCombinedResult->SizeY);

	// TODO: see comment in function
	DoReinitializationOverlapCheck(PatchToSource, TemporaryNativeHeightCopy->SizeX, TemporaryNativeHeightCopy->SizeY);

	ENQUEUE_RENDER_COMMAND(LandscapeTexturePatchReinitializeHeight)(
		[Source = InCombinedResult->GetResource(), Destination = TemporaryNativeHeightCopy->GetResource(),
		&PatchToSource, OffsetToApply](FRHICommandListImmediate& RHICmdList)
	{
		using namespace UE::Landscape;

		FRDGBuilder GraphBuilder(RHICmdList, RDG_EVENT_NAME("LandscapeTexturePatchReinitializeHeight"));

		FReinitializeLandscapePatchPS::FParameters* HeightmapResampleParams = GraphBuilder.AllocParameters<FReinitializeLandscapePatchPS::FParameters>();

		FRDGTextureRef HeightmapSource = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(Source->GetTexture2DRHI(), TEXT("ReinitializationSource")));
		FRDGTextureSRVRef SourceSRV = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(HeightmapSource, 0));
		HeightmapResampleParams->InSource = SourceSRV;
		HeightmapResampleParams->InSourceSampler = TStaticSamplerState<SF_Bilinear, AM_Wrap, AM_Wrap, AM_Wrap>::GetRHI();
		HeightmapResampleParams->InPatchToSource = PatchToSource;

		FRDGTextureRef DestinationTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(Destination->GetTexture2DRHI(), TEXT("ReinitializationDestination")));

		if (OffsetToApply != 0)
		{
			FRDGTextureRef TemporaryDestination = GraphBuilder.CreateTexture(DestinationTexture->Desc, TEXT("LandscapeTextureHeightPatchInputCopy"));
			HeightmapResampleParams->RenderTargets[0] = FRenderTargetBinding(TemporaryDestination, ERenderTargetLoadAction::ENoAction, /*InMipIndex = */0);

			FReinitializeLandscapePatchPS::AddToRenderGraph(GraphBuilder, HeightmapResampleParams, /*bHeightPatch*/ true);

			FOffsetHeightmapPS::FParameters* OffsetParams = GraphBuilder.AllocParameters<FOffsetHeightmapPS::FParameters>();

			FRDGTextureSRVRef InputSRV = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(TemporaryDestination, 0));
			OffsetParams->InHeightmap = InputSRV;
			OffsetParams->InHeightOffset = OffsetToApply;
			OffsetParams->RenderTargets[0] = FRenderTargetBinding(DestinationTexture, ERenderTargetLoadAction::ENoAction, /*InMipIndex = */0);

			FOffsetHeightmapPS::AddToRenderGraph(GraphBuilder, OffsetParams);
		}
		else
		{
			HeightmapResampleParams->RenderTargets[0] = FRenderTargetBinding(DestinationTexture, ERenderTargetLoadAction::ENoAction, /*InMipIndex = */0);
			FReinitializeLandscapePatchPS::AddToRenderGraph(GraphBuilder, HeightmapResampleParams, /*bHeightPatch*/ true);
		}

		GraphBuilder.Execute();
	});

	// The Modify() calls currently don't really help because we don't transact inside Render_Native. Maybe someday
	// we'll add that ability (though it sounds messy).
	UTexture2D* InternalTexture = HeightInternalData->GetInternalTexture();
	InternalTexture->Modify();
	FText ErrorMessage;
	if (TemporaryNativeHeightCopy->UpdateTexture(InternalTexture, CTF_Default, /*InAlphaOverride = */nullptr, /*InTextureChangingDelegate =*/ [](UTexture*) {}, &ErrorMessage))
	{
		check(InternalTexture->Source.GetFormat() == ETextureSourceFormat::TSF_BGRA8);
		InternalTexture->UpdateResource();
	}
	else
	{
		UE_LOGF(LogLandscapePatch, Error, "Couldn't copy heightmap render target to internal texture: %ls", *ErrorMessage.ToString());
	}
	InternalTexture->UpdateResource();

	if (IsValid(HeightInternalData->GetRenderTarget()))
	{
		HeightInternalData->GetRenderTarget()->Modify();
		HeightInternalData->CopyBackFromInternalTexture();
	}

	// Request a new landscape update to take into account the changes applied to the texture right away
	//  Defer it till next frame (ExecuteOnGameThread) since requesting an update while updating won't do anything
	ExecuteOnGameThread(TEXT("DeferredReinitializeHeightPatch"), [this] { RequestLandscapeUpdate(); });
}

void ULandscapeTexturePatch::ReinitializeWeightPatch(ULandscapeWeightPatchTextureInfo* PatchInfo, 
	FTextureResource* InputResource, FIntPoint ResourceSize, int32 SliceIndex, 
	const FTransform& LandscapeHeightmapToWorld)
{
	using namespace LandscapeTexturePatchLocals;

	if (!ensure(IsValid(PatchInfo) && InputResource))
	{
		return;
	}

	if (PatchInfo->SourceMode == ELandscapeTexturePatchSourceMode::TextureAsset)
	{
		const FString LayerNameString = PatchInfo->WeightmapLayerName.ToString();
		UE_LOGF(LogLandscapePatch, Warning, "ULandscapeTexturePatch: Cannot initialize weight layer %ls because source mode is an external texture.", *LayerNameString);
		return;
	}

	if (PatchInfo->SourceMode == ELandscapeTexturePatchSourceMode::None)
	{
		PatchInfo->SetSourceMode(ELandscapeTexturePatchSourceMode::InternalTexture);
	}
	else if (IsValid(PatchInfo->InternalData))
	{
		if (PatchInfo->SourceMode == ELandscapeTexturePatchSourceMode::InternalTexture && IsValid(PatchInfo->InternalData->GetInternalTexture()))
		{
			PatchInfo->InternalData->GetInternalTexture()->Modify();
		}
		else if (PatchInfo->SourceMode == ELandscapeTexturePatchSourceMode::TextureBackedRenderTarget && IsValid(PatchInfo->InternalData->GetRenderTarget()))
		{
			PatchInfo->InternalData->GetRenderTarget()->Modify();
		}
	}

	// Clear the default/shared weight patch alpha
	WeightPatchesAlphaSettings.SetAlphaSourceMode(ELandscapeTexturePatchAlphaSourceMode::None, this, /*bAlwaysMarkDiry =*/true);

	// Clear the alpha overrides
	PatchInfo->SetWeightPatchAlphaSourceMode(ELandscapeTexturePatchAlphaSourceMode::None, /*bInOverrideAlpha =*/false);
	
	if (!ensure(PatchInfo->InternalData))
	{
		return;
	}

	PatchInfo->InternalData->SetUseAlphaChannel(false);
	if (WeightPatchesBlendMode != ELandscapeTexturePatchBlendMode::AlphaBlend)
	{
		PatchInfo->bOverrideBlendMode = true;
		PatchInfo->OverrideBlendMode = ELandscapeTexturePatchBlendMode::AlphaBlend;
	}

	// We're going to copy directly to the associated render target. Make sure there is one for us to copy to.
	PatchInfo->InternalData->SetUseInternalTextureOnly(false, false);
	UTextureRenderTarget2D* RenderTarget = PatchInfo->InternalData->GetRenderTarget();
	if (!ensure(IsValid(RenderTarget)))
	{
		return;
	}

	FMatrix44f PatchToSource = GetPatchToHeightmapUVs(LandscapeHeightmapToWorld, 
		RenderTarget->SizeX, RenderTarget->SizeY, ResourceSize.X, ResourceSize.Y);

	// TODO: see comment in function
	DoReinitializationOverlapCheck(PatchToSource, RenderTarget->SizeX, RenderTarget->SizeY);

	ENQUEUE_RENDER_COMMAND(LandscapeTexturePatchReinitializeWeight)(
		[InputResource, SliceIndex, Destination = RenderTarget->GetResource(), &PatchToSource](FRHICommandListImmediate& RHICmdList)
	{
		using namespace UE::Landscape;

		FRDGBuilder GraphBuilder(RHICmdList, RDG_EVENT_NAME("LandscapeTexturePatchReinitializeWeight"));

		FReinitializeLandscapePatchPS::FParameters* ShaderParams = GraphBuilder.AllocParameters<FReinitializeLandscapePatchPS::FParameters>();

		if (SliceIndex < 0)
		{
			FRDGTextureRef SourceTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(InputResource->GetTexture2DRHI(), TEXT("ReinitializationSource")));
			ShaderParams->InSource = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(SourceTexture, 0));
		}
		else
		{
			FRDGTextureRef SourceTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(InputResource->GetTexture2DArrayRHI(), TEXT("ReinitializationSource")));
			FRDGTextureSRVDesc Desc = FRDGTextureSRVDesc::CreateForSlice(SourceTexture, SliceIndex);
			Desc.MipLevel = 0;
			Desc.NumMipLevels = 1;
			ShaderParams->InSource = GraphBuilder.CreateSRV(Desc);
		}

		ShaderParams->InSourceSampler = TStaticSamplerState<SF_Bilinear, AM_Wrap, AM_Wrap, AM_Wrap>::GetRHI();

		ShaderParams->InPatchToSource = PatchToSource;

		FRDGTextureRef DestinationTexture = GraphBuilder.RegisterExternalTexture(CreateRenderTarget(Destination->GetTexture2DRHI(), TEXT("ReinitializationDestination")));
		ShaderParams->RenderTargets[0] = FRenderTargetBinding(DestinationTexture, ERenderTargetLoadAction::ENoAction, /*InMipIndex = */0);
		FReinitializeLandscapePatchPS::AddToRenderGraph(GraphBuilder, ShaderParams, /*bHeightPatch*/ false);

		GraphBuilder.Execute();
	});

	PatchInfo->InternalData->SetUseInternalTextureOnly(PatchInfo->SourceMode == ELandscapeTexturePatchSourceMode::InternalTexture, true);

	// Request a new landscape update to take into account the changes applied to the texture right away
	//  Defer it till next frame (ExecuteOnGameThread) since requesting an update while updating won't do anything
	ExecuteOnGameThread(TEXT("DeferredReinitializeWeightPatch"), [this] { RequestLandscapeUpdate(); });
}

FMatrix44f ULandscapeTexturePatch::GetPatchToHeightmapUVs(const FTransform& LandscapeHeightmapToWorld,
	int32 PatchSizeX, int32 PatchSizeY, int32 HeightmapSizeX, int32 HeightmapSizeY) const
{
	using namespace LandscapeTexturePatchLocals;

	FVector2D FullPatchDimensions = bUseExternalTextureFix ? GetCoverageWithExtraPixel(FVector2D(PatchSizeX, PatchSizeY), GetUnscaledCoverage())
		: GetFullUnscaledWorldSize();

	FTransform PatchPixelToPatchLocal(FQuat4d::Identity, FVector3d(-FullPatchDimensions.X / 2, -FullPatchDimensions.Y / 2, 0),
		FVector3d(FullPatchDimensions.X / PatchSizeX, FullPatchDimensions.Y / PatchSizeY, 1));

	FTransform PatchToWorld = GetPatchToWorldTransform();

	FTransform LandscapeUVToWorld = LandscapeHeightmapToWorld;
	LandscapeUVToWorld.MultiplyScale3D(FVector3d(HeightmapSizeX, HeightmapSizeY, 1));

	// In unreal, matrix composition is done by multiplying the subsequent ones on the right, and the result
	// is transpose of what our shader will expect (because unreal right multiplies vectors by matrices).
	FMatrix44d PatchToLandscapeUVTransposed = PatchPixelToPatchLocal.ToMatrixWithScale() * PatchToWorld.ToMatrixWithScale()
		* LandscapeUVToWorld.ToInverseMatrixWithScale();
	return (FMatrix44f)PatchToLandscapeUVTransposed.GetTransposed();
}

bool ULandscapeTexturePatch::CanAffectHeightmap() const
{
	return (HeightSourceMode != ELandscapeTexturePatchSourceMode::None
		// If source mode is texture asset, we need to have an asset to read from
		&& (HeightSourceMode != ELandscapeTexturePatchSourceMode::TextureAsset || HeightTextureAsset))
		// If reinitializing, we need to read from the render call
		|| bReinitializeHeightOnNextRender;
}

bool ULandscapeTexturePatch::CanAffectWeightmap() const
{
	return Algo::AnyOf(WeightPatches, [this](const TObjectPtr<ULandscapeWeightPatchTextureInfo>& InWeightPatch) 
	{ 
		return IsValid(InWeightPatch) && WeightPatchCanRender(*InWeightPatch);
	});
}

bool ULandscapeTexturePatch::CanAffectWeightmapLayer(const FName& InLayerName) const
{
	return Algo::AnyOf(WeightPatches, [InLayerName, this](TObjectPtr<ULandscapeWeightPatchTextureInfo> InWeightPatch) 
	{
		return IsValid(InWeightPatch) && (InWeightPatch->WeightmapLayerName == InLayerName)
			&& WeightPatchCanRender(*InWeightPatch);
	}); 
}

bool ULandscapeTexturePatch::CanAffectVisibilityLayer() const
{
	return Algo::AnyOf(WeightPatches, [this](const TObjectPtr<ULandscapeWeightPatchTextureInfo>& InWeightPatch) 
	{ 
		return IsValid(InWeightPatch) && InWeightPatch->bEditVisibilityLayer 
			&& WeightPatchCanRender(*InWeightPatch);
	});
}

bool ULandscapeTexturePatch::WeightPatchCanRender(const ULandscapeWeightPatchTextureInfo& InWeightPatch) const
{
	return (InWeightPatch.SourceMode != ELandscapeTexturePatchSourceMode::None 
		// If source mode is texture asset, we need to have an asset to read from
		// It's valid for the weight patch alpha texture to be invalid, a default system texture will be supplied for the alpha
		&& (InWeightPatch.SourceMode != ELandscapeTexturePatchSourceMode::TextureAsset || InWeightPatch.TextureAsset))
		// If reinitializing, we need to read from the render call
		|| InWeightPatch.bReinitializeOnNextRender;
}

void ULandscapeTexturePatch::GetRenderDependencies(TSet<UObject*>& OutDependencies) const
{
	Super::GetRenderDependencies(OutDependencies);

	if (HeightSourceMode == ELandscapeTexturePatchSourceMode::InternalTexture
		&& HeightInternalData && HeightInternalData->GetInternalTexture())
	{
		OutDependencies.Add(HeightInternalData->GetInternalTexture());
	}
	else if (HeightSourceMode == ELandscapeTexturePatchSourceMode::TextureAsset 
		&& HeightTextureAsset)
	{
		OutDependencies.Add(HeightTextureAsset);
	}

	// Heightmap alpha texture
	if (HeightAlphaSettings.GetAlphaSourceMode() == ELandscapeTexturePatchAlphaSourceMode::TextureAsset)
	{
		OutDependencies.Add(HeightAlphaSettings.AlphaTextureAsset);
	}
	else if (HeightAlphaSettings.GetAlphaSourceMode() == ELandscapeTexturePatchAlphaSourceMode::InternalTexture
		&& HeightAlphaSettings.InternalAlphaData && HeightAlphaSettings.InternalAlphaData->GetInternalTexture())
	{
		OutDependencies.Add(HeightAlphaSettings.InternalAlphaData->GetInternalTexture());
	}

	// Weightmap alpha texture (Default alpha for all weight patches) 
	if (WeightPatchesAlphaSettings.GetAlphaSourceMode() == ELandscapeTexturePatchAlphaSourceMode::TextureAsset)
	{
		OutDependencies.Add(WeightPatchesAlphaSettings.AlphaTextureAsset);
	}
	else if (WeightPatchesAlphaSettings.GetAlphaSourceMode() == ELandscapeTexturePatchAlphaSourceMode::InternalTexture)
	{
		if (WeightPatchesAlphaSettings.InternalAlphaData && WeightPatchesAlphaSettings.InternalAlphaData->GetInternalTexture())
		{
			OutDependencies.Add(WeightPatchesAlphaSettings.InternalAlphaData->GetInternalTexture());
		}
	}

	// Individual weight patch data
	Algo::ForEach(WeightPatches, [&OutDependencies, this](const TObjectPtr<ULandscapeWeightPatchTextureInfo>& InWeightPatch)
		{ 
			if (IsValid(InWeightPatch))
			{
				if (InWeightPatch->SourceMode == ELandscapeTexturePatchSourceMode::InternalTexture
					&& InWeightPatch->InternalData && InWeightPatch->InternalData->GetInternalTexture())
				{
					OutDependencies.Add(InWeightPatch->InternalData->GetInternalTexture());
				}
				else if (InWeightPatch->SourceMode == ELandscapeTexturePatchSourceMode::TextureAsset
					&& InWeightPatch->TextureAsset)
				{
					OutDependencies.Add(InWeightPatch->TextureAsset);
				}

				// If this weight patch overrides the default alpha, add the additional dependencies
				if (InWeightPatch->bOverrideWeightAlphaSettings)
				{
					const FLandscapeTexturePatchAlphaSettings& PatchAlphaSettings = InWeightPatch->WeightPatchAlphaSettings;
					if (PatchAlphaSettings.GetAlphaSourceMode() == ELandscapeTexturePatchAlphaSourceMode::TextureAsset)
					{
						OutDependencies.Add(PatchAlphaSettings.AlphaTextureAsset);
					}
					else if (PatchAlphaSettings.GetAlphaSourceMode() == ELandscapeTexturePatchAlphaSourceMode::InternalTexture)
					{
						if (PatchAlphaSettings.InternalAlphaData && PatchAlphaSettings.InternalAlphaData->GetInternalTexture())
						{
							OutDependencies.Add(PatchAlphaSettings.InternalAlphaData->GetInternalTexture());
						}
					}
				}
			}
		});
}

TStructOnScope<FActorComponentInstanceData> ULandscapeTexturePatch::GetComponentInstanceData() const
{
	// There are currently various issues with blueprints and instanced sub objects, and
	//  one of them causes undo to be severely broken for transactable instanced objects
	//  inside a blueprint actor component: UE-225445
	// As it happens, one workaround is to not have the objects be transactable. So for
	//  now, we temporarily make all instanced objects not transactable while doing instance
	//  data serialization (when it theoretically shouldn't matter anyway).

	auto SetObjectTransactionalFlag = [](UObject* Object, bool bOn)
	{
		if (!Object)
		{
			return;
		}
		if (bOn)
		{
			Object->SetFlags(RF_Transactional);
		}
		else
		{
			Object->ClearFlags(RF_Transactional);
		}
	};
	auto MakeTemporarilyNonTransactional = [this, SetObjectTransactionalFlag](bool bInIsTemporarilyNonTransactional)
	{
		SetObjectTransactionalFlag(const_cast<ULandscapeTexturePatch*>(this), !bInIsTemporarilyNonTransactional);

		// ULandscapeTextureBackedRenderTargetBase::SetTemporarilyNonTransactional will internally make all of its sub-objects transactional or non-transactionals as well
		//  as prevent the RF_Transactional flags from inadvertently being set by one of the functions called in the duplication process (e.g. ULandscapeTextureBackedRenderTargetBase::PostLoad)
		if (IsValid(HeightInternalData))
		{
			HeightInternalData->SetTemporarilyNonTransactional(bInIsTemporarilyNonTransactional);
		}

		// Default heightmap alpha
		if (IsValid(HeightAlphaSettings.InternalAlphaData))
		{
			HeightAlphaSettings.InternalAlphaData->SetTemporarilyNonTransactional(bInIsTemporarilyNonTransactional);
		}

		// Default weightmap alpha
		if (IsValid(WeightPatchesAlphaSettings.InternalAlphaData))
		{
			WeightPatchesAlphaSettings.InternalAlphaData->SetTemporarilyNonTransactional(bInIsTemporarilyNonTransactional);
		}

		for (const TObjectPtr<ULandscapeWeightPatchTextureInfo>& WeightPatch : WeightPatches)
		{
			if (IsValid(WeightPatch))
			{
				SetObjectTransactionalFlag(WeightPatch, !bInIsTemporarilyNonTransactional);

				if (IsValid(WeightPatch->InternalData))
				{
					WeightPatch->InternalData->SetTemporarilyNonTransactional(bInIsTemporarilyNonTransactional);
				}
				
				// Override weightmap alpha
				if (WeightPatch->bOverrideWeightAlphaSettings && IsValid(WeightPatch->WeightPatchAlphaSettings.InternalAlphaData))
				{
					WeightPatch->WeightPatchAlphaSettings.InternalAlphaData->SetTemporarilyNonTransactional(bInIsTemporarilyNonTransactional);
				}
			}
		}
	};
	
	MakeTemporarilyNonTransactional(true);
	TStructOnScope<FActorComponentInstanceData> ToReturn = Super::GetComponentInstanceData();
	MakeTemporarilyNonTransactional(false);

	return ToReturn;
}

void ULandscapeTexturePatch::CheckForErrors()
{
	auto DidPatchUseATextureAsset = [this]()
	{
		if (HeightSourceMode == ELandscapeTexturePatchSourceMode::TextureAsset && HeightTextureAsset)
		{
			return true;
		}
		for (const TObjectPtr<ULandscapeWeightPatchTextureInfo>& WeightPatch : WeightPatches)
		{
			if (IsValid(WeightPatch) && WeightPatch->SourceMode == ELandscapeTexturePatchSourceMode::TextureAsset
				&& WeightPatch->TextureAsset)
			{
				return true;
			}
		}
		return false;
	};

	if (!IsTemplate()
		&& GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::LandscapeTexturePatchUsesTextureAssetResolution
		&& DidPatchUseATextureAsset())
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("Package"), FText::FromString(*GetNameSafe(GetPackage())));
		Arguments.Add(TEXT("Actor"), FText::FromString(*GetNameSafe(this)));

		FMessageLog("MapCheck").Warning()
			->AddToken(FUObjectToken::Create(this))
			->AddToken(FTextToken::Create(FText::Format(LOCTEXT("PatchAlignmentMightBeAdjusted", "Patch was saved with a texture "
				"asset when alignment code for texture assets had a minor bug. Verify that difference in landscape is negligible, and "
				"resave the patch. "
				"(Package: {Package}, Actor: {Actor})."), Arguments)))
			->AddToken(FActionToken::Create(LOCTEXT("MarkDirtyButton", "Mark dirty"), FText(),
				FOnActionTokenExecuted::CreateWeakLambda(this, [this]()
					{
						MarkPackageDirty();
					})));
	}
}

#endif

void ULandscapeTexturePatch::SnapToLandscape()
{
#if WITH_EDITOR
	if (!Landscape.IsValid())
	{
		return;
	}

	Modify();

	FTransform LandscapeTransform = Landscape->GetTransform();
	FTransform PatchTransform = GetComponentTransform();

	FQuat LandscapeRotation = LandscapeTransform.GetRotation();
	FQuat PatchRotation = PatchTransform.GetRotation();

	// Get rotation of patch relative to landscape
	FQuat PatchRotationRelativeLandscape = LandscapeRotation.Inverse() * PatchRotation;

	// Get component of that relative rotation that is around the landscape Z axis.
	double RadiansAroundZ = PatchRotationRelativeLandscape.GetTwistAngle(FVector::ZAxisVector);

	// Round that rotation to nearest 90 degree increment
	int32 Num90DegreeRotations = FMath::RoundToDouble(RadiansAroundZ / FMathd::HalfPi);
	double NewRadiansAroundZ = Num90DegreeRotations * FMathd::HalfPi;

	// Now adjust the patch transform.
	FQuat NewPatchRotation = FQuat(FVector::ZAxisVector, NewRadiansAroundZ) * LandscapeRotation;
	SetWorldRotation(NewPatchRotation);

	// Once we have the rotation adjusted, we need to adjust the patch size and positioning.
	// However don't bother if either the patch or landscape scale is 0. We might still be able
	// to align in one of the axes in such a case, but it is not worth the code complexity for
	// a broken use case.
	FVector LandscapeScale = Landscape->GetTransform().GetScale3D();
	FVector PatchScale = GetComponentTransform().GetScale3D();
	if (LandscapeScale.X == 0 || LandscapeScale.Y == 0)
	{
		UE_LOGF(LogLandscapePatch, Warning, "ULandscapeTexturePatch::SnapToLandscape: Landscape target "
			"for patch had a zero scale in one of the dimensions. Skipping aligning position.");
		return;
	}
	if (PatchScale.X == 0 || PatchScale.Y == 0)
	{
		UE_LOGF(LogLandscapePatch, Warning, "ULandscapeTexturePatch::SnapToLandscape: Patch "
			"had a zero scale in one of the dimensions. Skipping aligning position.");
		return;
	}

	// Start by adjusting size to be a multiple of landscape quad size.
	double PatchExtentX = PatchScale.X * UnscaledPatchCoverage.X;
	double PatchExtentY = PatchScale.Y * UnscaledPatchCoverage.Y;
	if (Num90DegreeRotations % 2)
	{
		// Relative to the landscape, our length and width are backwards...
		Swap(PatchExtentX, PatchExtentY);
	}

	int32 LandscapeQuadsX = FMath::RoundToInt(PatchExtentX / LandscapeScale.X);
	int32 LandscapeQuadsY = FMath::RoundToInt(PatchExtentY / LandscapeScale.Y);

	double NewPatchExtentX = LandscapeQuadsX * LandscapeScale.X;
	double NewPatchExtentY = LandscapeQuadsY * LandscapeScale.Y;
	if (Num90DegreeRotations % 2)
	{
		Swap(NewPatchExtentX, NewPatchExtentY);
	}
	UnscaledPatchCoverage = FVector2D(NewPatchExtentX / PatchScale.X, NewPatchExtentY / PatchScale.Y);

	// Now adjust the center of the patch. This gets snapped to either integer or integer + 0.5 increments
	// in landscape coordinates depending on whether patch length/width is odd or even in landscape coordinates.

	FVector PatchCenterInLandscapeCoordinates = LandscapeTransform.InverseTransformPosition(GetComponentLocation());
	double NewPatchCenterX, NewPatchCenterY;
	if (LandscapeQuadsX % 2)
	{
		NewPatchCenterX = FMath::RoundToDouble(PatchCenterInLandscapeCoordinates.X + 0.5) - 0.5;
	}
	else
	{
		NewPatchCenterX = FMath::RoundToDouble(PatchCenterInLandscapeCoordinates.X);
	}
	if (LandscapeQuadsY % 2)
	{
		NewPatchCenterY = FMath::RoundToDouble(PatchCenterInLandscapeCoordinates.Y + 0.5) - 0.5;
	}
	else
	{
		NewPatchCenterY = FMath::RoundToDouble(PatchCenterInLandscapeCoordinates.Y);
	}

	FVector NewCenterInLandscape(NewPatchCenterX, NewPatchCenterY, PatchCenterInLandscapeCoordinates.Z);
	SetWorldLocation(LandscapeTransform.TransformPosition(NewCenterInLandscape));
	RequestLandscapeUpdate();
#endif // WITH_EDITOR
}

void ULandscapeTexturePatch::SetResolution(FVector2D ResolutionIn)
{
	int32 DesiredX = FMath::Max(1, ResolutionIn.X);
	int32 DesiredY = FMath::Max(1, ResolutionIn.Y);

	if (DesiredX == ResolutionX && DesiredY == ResolutionY)
	{
		return;
	}
	Modify();

	ResolutionX = DesiredX;
	ResolutionY = DesiredY;
	InitTextureSizeX = ResolutionX;
	InitTextureSizeY = ResolutionY;

	auto ResizePatch = [DesiredX, DesiredY](ELandscapeTexturePatchSourceMode SourceMode, ULandscapeTextureBackedRenderTargetBase* InternalData)
	{
		// Deal with height first
		if (SourceMode == ELandscapeTexturePatchSourceMode::TextureAsset || SourceMode == ELandscapeTexturePatchSourceMode::None)
		{
			return;
		}
		else if (ensure(IsValid(InternalData)))
		{
			InternalData->SetResolution(DesiredX, DesiredY);
		}
	};

	ResizePatch(HeightSourceMode, HeightInternalData);
	// Height alpha data
	ResizePatch(LandscapeTexturePatchLocals::AlphaModeToSourceMode(HeightAlphaSettings.GetAlphaSourceMode()), HeightAlphaSettings.InternalAlphaData);
	// Default weight alpha data 
	ResizePatch(LandscapeTexturePatchLocals::AlphaModeToSourceMode(WeightPatchesAlphaSettings.GetAlphaSourceMode()), WeightPatchesAlphaSettings.InternalAlphaData);

	for (const TObjectPtr<ULandscapeWeightPatchTextureInfo>& WeightPatch : WeightPatches)
	{
		if (IsValid(WeightPatch))
		{
			ResizePatch(WeightPatch->SourceMode, WeightPatch->InternalData);

			// Override weight alpha data
			if (WeightPatch->bOverrideWeightAlphaSettings)
			{
				ResizePatch(LandscapeTexturePatchLocals::AlphaModeToSourceMode(WeightPatch->WeightPatchAlphaSettings.GetAlphaSourceMode()), WeightPatch->WeightPatchAlphaSettings.InternalAlphaData);
			}
		}
	}
}

FVector2D ULandscapeTexturePatch::GetFullUnscaledWorldSize() const
{
	using namespace LandscapeTexturePatchLocals;

	return GetCoverageWithExtraPixel(GetResolution(), UnscaledPatchCoverage);
}

FTransform ULandscapeTexturePatch::GetPatchToWorldTransform() const
{
	FTransform PatchToWorld = GetComponentTransform();

	if (Landscape.IsValid())
	{
		FRotator3d PatchRotator = PatchToWorld.GetRotation().Rotator();
		FRotator3d LandscapeRotator = Landscape->GetTransform().GetRotation().Rotator();
		PatchToWorld.SetRotation(FRotator3d(LandscapeRotator.Pitch, PatchRotator.Yaw, LandscapeRotator.Roll).Quaternion());
	}

	return PatchToWorld;
}

bool ULandscapeTexturePatch::GetInitResolutionFromLandscape(float ResolutionMultiplierIn, FVector2D& ResolutionOut) const
{
	if (!Landscape.IsValid())
	{
		return false;
	}

	ResolutionOut = FVector2D::One();

	FVector LandscapeScale = Landscape->GetTransform().GetScale3D();
	// We go off of the larger dimension so that our patch works in different rotations.
	double LandscapeQuadSize = FMath::Max(FMath::Abs(LandscapeScale.X), FMath::Abs(LandscapeScale.Y));

	if (LandscapeQuadSize > 0)
	{
		double PatchQuadSize = LandscapeQuadSize;
		PatchQuadSize /= (ResolutionMultiplierIn > 0 ? ResolutionMultiplierIn : 1);

		FVector PatchScale = GetComponentTransform().GetScale3D();
		double NumQuadsX = FMath::Abs(UnscaledPatchCoverage.X * PatchScale.X / PatchQuadSize);
		double NumQuadsY = FMath::Abs(UnscaledPatchCoverage.Y * PatchScale.Y / PatchQuadSize);

		ResolutionOut = FVector2D(
			FMath::Max(1, FMath::CeilToInt(NumQuadsX) + 1),
			FMath::Max(1, FMath::CeilToInt(NumQuadsY) + 1)
		);

		return true;
	}
	return false;
}

#if WITH_EDITOR
void ULandscapeTexturePatch::PostLoad()
{
	Super::PostLoad();

	// Handle weight/height patch alpha channel and blend mode deprecation
	if (GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::LandscapeTexturePatchAlphaOverride)
	{
		// Weight patches used the height blend mode as the default fallback. Copy the legacy height fallback to the weight patches default blend mode
		WeightPatchesBlendMode = BlendMode;
	
		// Reset the height texture channel if the user is using native encoding/internal texture
		if (HeightSourceMode == ELandscapeTexturePatchSourceMode::InternalTexture || HeightEncoding == ELandscapeTextureHeightPatchEncoding::NativePackedHeight)
		{
			HeightTextureChannel = ELandscapeTexturePatchTextureChannel::None;
		}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// Convert heightmap bUseAlphaChannel legacy flag to the new alpha enum
		if (bUseTextureAlphaForHeight_DEPRECATED)
		{
			SetHeightAlphaSourceMode(ELandscapeTexturePatchAlphaSourceMode::SourceTextureChannel, /*bAlwaysMarkDirty =*/false);
		}

PRAGMA_ENABLE_DEPRECATION_WARNINGS
		
		// Convert weightmap bUseAlphaChannel legacy flags to the new alpha enum
		for (TObjectPtr<ULandscapeWeightPatchTextureInfo> WeightPatch : WeightPatches)
		{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (WeightPatch && WeightPatch->bUseAlphaChannel_DEPRECATED)
			{
				// Set the override alpha source mode
				WeightPatch->SetWeightPatchAlphaSourceMode(ELandscapeTexturePatchAlphaSourceMode::SourceTextureChannel,
					/*bInOverrideAlpha =*/true,
					/*bAlwaysMarkDirty = */ false);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
		}
	}
}

void ULandscapeTexturePatch::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	using namespace LandscapeTexturePatchLocals;

	if (PropertyChangedEvent.Property)
	{
		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ULandscapeTexturePatch, DetailPanelHeightSourceMode))
		{
			// When changing source mode in the detail panel to a render target, we need to know the format to use, particularly 
			// whether we need an alpha channel
			if ((DetailPanelHeightSourceMode == ELandscapeTexturePatchSourceMode::TextureBackedRenderTarget
					// This also affects an internal texture if we're copying from a texture asset, because we copy through render target
					|| DetailPanelHeightSourceMode == ELandscapeTexturePatchSourceMode::InternalTexture)
				// However we don't want to touch the format if we started with a render target source mode, because that would clear
				// the render target before we can copy it to an internal texture (if that's what we're switching to).
				&& HeightSourceMode != ELandscapeTexturePatchSourceMode::TextureBackedRenderTarget)
			{
				ResetHeightRenderTargetFormat();
			}
			SetHeightSourceMode(DetailPanelHeightSourceMode);
		}
		else if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ULandscapeTexturePatch, HeightEncoding))
		{
			ResetHeightEncodingMode(HeightEncoding);
		}
		else if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ULandscapeTexturePatch, WeightPatches))
		{
			// In certain cases, changes to the internals of a weight info object trigger a PostEditChangeProperty
			//  on the patch instead of the info object. For instance this happens when editing the objects in the
			//  blueprint editor and propagating the change to an instance (something that frequently does not work
			//  due to propagation being unreliable for this array, see comment on WeightPatches).
			for (TObjectPtr<ULandscapeWeightPatchTextureInfo>& WeightPatch : WeightPatches)
			{
				if (IsValid(WeightPatch))
				{
					WeightPatch->SetSourceMode(WeightPatch->DetailPanelSourceMode);
				}
			}
		}
		else if (PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(ULandscapeTexturePatch, WeightPatchesAlphaSettings))
		{
			// Changing the default weightmap alpha source mode
			if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(FLandscapeTexturePatchAlphaSettings, DetailPanelAlphaSourceMode))
			{
				SetWeightPatchAlphaSourceMode(FName(), WeightPatchesAlphaSettings.DetailPanelAlphaSourceMode);
			}
		}
		else if (PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(ULandscapeTexturePatch, HeightAlphaSettings))
		{
			if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(FLandscapeTexturePatchAlphaSettings, DetailPanelAlphaSourceMode))
			{
				SetHeightAlphaSourceMode(HeightAlphaSettings.DetailPanelAlphaSourceMode);
			}
		}
		else if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(FLandscapeTexturePatchEncodingSettings, ZeroInEncoding)
			|| PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(FLandscapeTexturePatchEncodingSettings, WorldSpaceEncodingScale))
		{
			UpdateHeightConvertToNativeParamsIfNeeded();
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void ULandscapeWeightPatchTextureInfo::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	using namespace LandscapeTexturePatchLocals;

	if (PropertyChangedEvent.Property)
	{
		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(ULandscapeWeightPatchTextureInfo, DetailPanelSourceMode)
			&& DetailPanelSourceMode != SourceMode)
		{
			SetSourceMode(DetailPanelSourceMode);
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void ULandscapeWeightPatchTextureInfo::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	FProperty* Property = PropertyChangedEvent.Property;
	FProperty* MemberProperty = PropertyChangedEvent.PropertyChain.GetActiveMemberNode() ? PropertyChangedEvent.PropertyChain.GetActiveMemberNode()->GetValue() : nullptr;

	if (Property && MemberProperty)
	{
		// Changing the alpha settings for a specific weight patch
		if (MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(ULandscapeWeightPatchTextureInfo, WeightPatchAlphaSettings))
		{
			if (Property->GetFName() == GET_MEMBER_NAME_CHECKED(FLandscapeTexturePatchAlphaSettings, DetailPanelAlphaSourceMode))
			{
				SetWeightPatchAlphaSourceMode(WeightPatchAlphaSettings.DetailPanelAlphaSourceMode, bOverrideWeightAlphaSettings);
			}
			else
			{
				// When bOverrideWeightAlphaSettings changes the property does not get passed to this function correctly
				// As a fallback, clear the internal data whenever the weight patch no longer overrides the default
				if (!bOverrideWeightAlphaSettings)
				{
					SetWeightPatchAlphaSourceMode(ELandscapeTexturePatchAlphaSourceMode::None, /*bInOverrideAlpha =*/false);
				}
			}
		}
	}
}

void ULandscapeWeightPatchTextureInfo::PreDuplicate(FObjectDuplicationParameters& DupParams)
{
	// TODO: It seems like this whole overload shouldn't be necessary, because we should get PreDuplicate calls
	// on InternalData. However for reasons that I have yet to undertand, those calls are not made. It seems like
	// there is different behavior for an array of instanced classes containing instanced properties...

	Super::PreDuplicate(DupParams);

	if (SourceMode == ELandscapeTexturePatchSourceMode::TextureBackedRenderTarget && InternalData)
	{
		InternalData->CopyToInternalTexture();
	}

	// Handle weightmap alpha override
	if (bOverrideWeightAlphaSettings)
	{
		if (WeightPatchAlphaSettings.GetAlphaSourceMode() == ELandscapeTexturePatchAlphaSourceMode::TextureBackedRenderTarget && WeightPatchAlphaSettings.InternalAlphaData)
		{
			WeightPatchAlphaSettings.InternalAlphaData->CopyToInternalTexture();
		}
	}
}
#endif // WITH_EDITOR

void ULandscapeTexturePatch::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
}

void ULandscapeWeightPatchTextureInfo::SetSourceMode(ELandscapeTexturePatchSourceMode NewMode)
{
#if WITH_EDITOR

	if (SourceMode == NewMode)
	{
		return;
	}
	Modify();

	if (!IsTemplate())
	{
		TransitionSourceModeInternal(SourceMode, NewMode);
	}
	// In a template, it is not safe to try to allocate a texture, etc. All we do is clear out the
	// texture asset pointer if it is not needed, to avoid referencing assets unnecessarily.
	else if (SourceMode != ELandscapeTexturePatchSourceMode::TextureAsset)
	{
		TextureAsset = nullptr;
	}

	SourceMode = NewMode;
	DetailPanelSourceMode = NewMode;
#endif // WITH_EDITOR
}

void ULandscapeWeightPatchTextureInfo::SetWeightPatchAlphaSourceMode(ELandscapeTexturePatchAlphaSourceMode InSourceMode, bool bInOverrideAlpha, bool bAlwaysMarkDirty/*=true*/)
{
#if WITH_EDITOR
	// Early return if override and source do not change
	if (bOverrideWeightAlphaSettings == bInOverrideAlpha && WeightPatchAlphaSettings.GetAlphaSourceMode() == InSourceMode)
	{
		return;
	}

	Modify(bAlwaysMarkDirty);
	bOverrideWeightAlphaSettings = bInOverrideAlpha;

	// When bInOverrideAlpha is false, the weightmap layer will fallback to the shared weightmap alpha. Reset the override source mode to clear internal data
	if (bOverrideWeightAlphaSettings)
	{
		WeightPatchAlphaSettings.SetAlphaSourceMode(InSourceMode, this);
	}
	else
	{
		WeightPatchAlphaSettings.SetAlphaSourceMode(ELandscapeTexturePatchAlphaSourceMode::None, this);
	}

	// Weightmap alpha channel is only utilized when SourceTextureChannel (weightmap's alpha) is selected. All other alpha modes generate a second alpha specific texture
	if (InternalData)
	{
		InternalData->SetUseAlphaChannel(InSourceMode == ELandscapeTexturePatchAlphaSourceMode::SourceTextureChannel);
	}
#endif // WITH_EDITOR
}

void ULandscapeWeightPatchTextureInfo::SetWeightPatchTextureChannel(ELandscapeTexturePatchTextureChannel InTextureChannel, bool bAlwaysMarkDirty)
{
#if WITH_EDITOR
	if (WeightPatchTextureChannel == InTextureChannel)
	{
		return;
	}
	Modify(bAlwaysMarkDirty);

	WeightPatchTextureChannel = InTextureChannel;
#endif // WITH_EDITOR
}

void ULandscapeWeightPatchTextureInfo::SetWeightPatchAlphaTextureChannel(ELandscapeTexturePatchTextureChannel InTextureChannel, bool bAlwaysMarkDirty)
{
#if WITH_EDITOR
	if (WeightPatchAlphaSettings.AlphaTextureChannel == InTextureChannel)
	{
		return;
	}
	Modify(bAlwaysMarkDirty);

	WeightPatchAlphaSettings.AlphaTextureChannel = InTextureChannel;
#endif // WITH_EDITOR
}

void ULandscapeWeightPatchTextureInfo::SetWeightPatchTextureTransform(const FLandscapeTexturePatchTextureTransform& InTextureTransform, bool bAlwaysMarkDirty)
{
#if WITH_EDITOR
	if (WeightTextureUVTransform == InTextureTransform)
	{
		return;
	}
	Modify(bAlwaysMarkDirty);

	WeightTextureUVTransform = InTextureTransform;
#endif // WITH_EDITOR
}

void ULandscapeWeightPatchTextureInfo::SetWeightPatchAlphaTextureTransform(const FLandscapeTexturePatchTextureTransform& InTextureTransform, bool bAlwaysMarkDirty)
{
#if WITH_EDITOR
	if (WeightPatchAlphaSettings.AlphaTextureUVTransform == InTextureTransform)
	{
		return;
	}
	Modify(bAlwaysMarkDirty);

	WeightPatchAlphaSettings.AlphaTextureUVTransform = InTextureTransform;
#endif // WITH_EDITOR
}

#if WITH_EDITOR
void ULandscapeWeightPatchTextureInfo::TransitionSourceModeInternal(ELandscapeTexturePatchSourceMode OldMode, ELandscapeTexturePatchSourceMode NewMode)
{
	using namespace LandscapeTexturePatchLocals;

	FVector2D Resolution(1, 1);
	if (ULandscapeTexturePatch* OwningPatch = Cast<ULandscapeTexturePatch>(GetOuter()))
	{
		Resolution = OwningPatch->GetResolution();
	}

	TransitionSourceMode<ULandscapeWeightTextureBackedRenderTarget>(SourceMode, NewMode, TextureAsset, InternalData, WeightPatchTextureChannel, [&Resolution, this]()
	{
		ULandscapeWeightTextureBackedRenderTarget* InternalDataToReturn = NewObject<ULandscapeWeightTextureBackedRenderTarget>(this);
		// Only set the transactional flag on that new sub-object if we are (it can happen that we are made temporarily non-transactional in which case, we don't want
		//  one of its sub-objects to become transactional) : 
		if (GetFlags() & RF_Transactional)
		{
			InternalDataToReturn->SetFlags(RF_Transactional);
		}
		InternalDataToReturn->SetResolution(Resolution.X, Resolution.Y);
		return InternalDataToReturn;
	});
}
#endif

void ULandscapeTexturePatch::SetHeightSourceMode(ELandscapeTexturePatchSourceMode NewMode)
{
#if WITH_EDITOR
	using namespace LandscapeTexturePatchLocals;

	if (HeightSourceMode == NewMode)
	{
		return;
	}
	Modify();

	if (!IsTemplate())
	{
		TransitionHeightSourceModeInternal(HeightSourceMode, NewMode);
	}
	// In a template, it is not safe to try to allocate a texture, etc. All we do is clear out the
	// texture asset pointer if it is not needed, to avoid referencing assets unnecessarily.
	else if (HeightSourceMode != ELandscapeTexturePatchSourceMode::TextureAsset)
	{
		HeightTextureAsset = nullptr;
	}

	HeightSourceMode = NewMode;
	DetailPanelHeightSourceMode = NewMode;
#endif // WITH_EDITOR
}

void ULandscapeTexturePatch::SetHeightTextureTransform(const FLandscapeTexturePatchTextureTransform& InTextureTransform, bool bAlwaysMarkDirty)
{
#if WITH_EDITOR
	if (HeightTextureUVTransform == InTextureTransform)
	{
		return;
	}
	Modify(bAlwaysMarkDirty);

	HeightTextureUVTransform = InTextureTransform;
#endif // WITH_EDITOR
}

void ULandscapeTexturePatch::SetHeightTextureChannel(ELandscapeTexturePatchTextureChannel InTextureChannel, bool bAlwaysMarkDirty)
{
#if WITH_EDITOR
	if (HeightTextureChannel == InTextureChannel)
	{
		return;
	}

	// When the height uses internal texture / native packed height we limit the user to RG texture color channels
	if (HeightSourceMode == ELandscapeTexturePatchSourceMode::InternalTexture || HeightEncoding == ELandscapeTextureHeightPatchEncoding::NativePackedHeight)
	{
		Modify(bAlwaysMarkDirty);
		HeightTextureChannel = ELandscapeTexturePatchTextureChannel::None;
		return;
	}

	Modify(bAlwaysMarkDirty);
	HeightTextureChannel = InTextureChannel;
#endif // WITH_EDITOR
}

void ULandscapeTexturePatch::SetHeightAlphaSourceMode(ELandscapeTexturePatchAlphaSourceMode InNewMode, bool bAlwaysMarkDirty/*=true*/)
{
#if WITH_EDITOR
	HeightAlphaSettings.SetAlphaSourceMode(InNewMode, this, bAlwaysMarkDirty);
#endif // WITH_EDITOR
}

void ULandscapeTexturePatch::SetHeightAlphaTextureTransform(const FLandscapeTexturePatchTextureTransform& InTextureTransform, bool bAlwaysMarkDirty)
{
#if WITH_EDITOR
	if (HeightAlphaSettings.AlphaTextureUVTransform == InTextureTransform)
	{
		return;
	}
	Modify(bAlwaysMarkDirty);

	HeightAlphaSettings.AlphaTextureUVTransform = InTextureTransform;
#endif // WITH_EDITOR
}

void ULandscapeTexturePatch::SetHeightAlphaTextureChannel(ELandscapeTexturePatchTextureChannel InTextureChannel, bool bAlwaysMarkDirty)
{
#if WITH_EDITOR
	if (HeightAlphaSettings.AlphaTextureChannel == InTextureChannel)
	{
		return;
	}
	Modify(bAlwaysMarkDirty);

	HeightAlphaSettings.AlphaTextureChannel = InTextureChannel;
#endif // WITH_EDITOR
}

#if WITH_EDITOR
void ULandscapeTexturePatch::TransitionHeightSourceModeInternal(ELandscapeTexturePatchSourceMode OldMode, ELandscapeTexturePatchSourceMode NewMode)
{
	using namespace LandscapeTexturePatchLocals;

	TransitionSourceMode<ULandscapeHeightTextureBackedRenderTarget>(HeightSourceMode, NewMode, HeightTextureAsset, HeightInternalData, HeightTextureChannel, [this]()
	{
		ULandscapeHeightTextureBackedRenderTarget* InternalDataToReturn = NewObject<ULandscapeHeightTextureBackedRenderTarget>(this);
		// Only set the transactional flag on that new sub-object if we are (it can happen that we are made temporarily non-transactional in which case, we don't want
		//  one of its sub-objects to become transactional) : 
		if (GetFlags() & RF_Transactional)
		{
			InternalDataToReturn->SetFlags(RF_Transactional);
		}
		InternalDataToReturn->SetResolution(ResolutionX, ResolutionY);
		InternalDataToReturn->SetFormat(HeightRenderTargetFormat);
		InternalDataToReturn->ConversionParams = GetHeightConvertToNativeParams();

		return InternalDataToReturn;
	});

	// When the Height Patch data uses InternalTexture mode/Native encoding we explicitly use the RG texture color channels
	if (NewMode == ELandscapeTexturePatchSourceMode::InternalTexture || HeightEncoding == ELandscapeTextureHeightPatchEncoding::NativePackedHeight)
	{
		HeightTextureChannel = ELandscapeTexturePatchTextureChannel::None;
	}
}
#endif

UTextureRenderTarget2D* ULandscapeTexturePatch::GetHeightAlphaRenderTarget(bool bMarkDirty)
{
#if WITH_EDITOR
	if (IsTemplate())
	{
		return nullptr;
	}

	if (bMarkDirty)
	{
		MarkPackageDirty();
	}

	return HeightAlphaSettings.GetAlphaRenderTarget(this, bMarkDirty);
#else
	return nullptr;
#endif
}

void ULandscapeTexturePatch::SetHeightTextureAsset(UTexture* TextureIn)
{
	if (TextureIn && TextureIn->VirtualTextureStreaming != 0)
	{
		UE_LOGF(LogLandscapePatch, Error, "ULandscapeTexturePatch::SetHeightTextureAsset: Invalid texture. Virtual textures are not supported.");
		return;
	}
	HeightTextureAsset = TextureIn;
}

UTextureRenderTarget2D* ULandscapeTexturePatch::GetHeightRenderTarget(bool bMarkDirty)
{
#if WITH_EDITOR

	if (IsTemplate())
	{
		return nullptr;
	}

	if (bMarkDirty)
	{
		MarkPackageDirty();
	}

	// In templates (i.e. in blueprint editor), it's not safe to create textures, so if we are an instantiation
	//  of a blueprint, we may not yet have the internal render target allocated. It might seem like a good idea
	//  to do this in OnComponentCreated, but that causes default construction script instance data application
	//  to see the data as modified, and prevents it from being carried over properly (see usage of GetUCSModifiedProperties
	//  in ComponentInstanceDataCache.cpp). Doing it in ApplyComponentInstanceData also seems to be a good idea at
	//  first, but we can't do it in ECacheApplyPhase::PostSimpleConstructionScript for the same reason as OnComponentModified,
	//  and doing it in ECacheApplyPhase::PostUserConstructionScript is too late because the user may want to write
	//  to the render target in the user construction script.
	// So, we do this allocation right when the render target is requested.
	if (HeightSourceMode == ELandscapeTexturePatchSourceMode::TextureBackedRenderTarget)
	{
		if (!HeightInternalData || !HeightInternalData->GetRenderTarget())
		{
			TransitionHeightSourceModeInternal(ELandscapeTexturePatchSourceMode::None, HeightSourceMode);
		}

		return ensure(HeightInternalData) ? HeightInternalData->GetRenderTarget() : nullptr;
	}
#endif

	return nullptr;
}

UTexture2D* ULandscapeTexturePatch::GetHeightInternalTexture()
{
#if WITH_EDITOR

	if (IsTemplate())
	{
		return nullptr;
	}

	// Allocate data if needed (see comment in GetHeightRenderTarget)
	if (HeightSourceMode == ELandscapeTexturePatchSourceMode::TextureBackedRenderTarget
		|| HeightSourceMode == ELandscapeTexturePatchSourceMode::InternalTexture)
	{
		if (!HeightInternalData || !HeightInternalData->GetInternalTexture())
		{
			TransitionHeightSourceModeInternal(ELandscapeTexturePatchSourceMode::None, HeightSourceMode);
		}

		return ensure(HeightInternalData) ? HeightInternalData->GetInternalTexture() : nullptr;
	}
#endif

	return nullptr;
}

UTextureRenderTarget2D* ULandscapeTexturePatch::GetHeightAlphaRenderTarget()
{
#if WITH_EDITOR
	return HeightAlphaSettings.GetAlphaRenderTarget(this);
#else
	return nullptr;
#endif // WITH_EDITOR
}

UTexture2D* ULandscapeTexturePatch::GetHeightAlphaInternalTexture()
{
#if WITH_EDITOR
	return HeightAlphaSettings.GetAlphaInternalTexture(this);
#else
	return nullptr;
#endif // WITH_EDITOR
}

void ULandscapeTexturePatch::UpdateHeightConvertToNativeParamsIfNeeded()
{
#if WITH_EDITOR
	if (HeightInternalData)
	{
		FLandscapeHeightPatchConvertToNativeParams ConversionParams = GetHeightConvertToNativeParams();
		if (ConversionParams.HeightScale == 0)
		{
			// If the scale is 0, then storing in the texture would lose the data we have,
			// so keep whatever the previous storage encoding was if nonzero, otherwise set to 1.
			ConversionParams.HeightScale = HeightInternalData->ConversionParams.HeightScale != 0 ? HeightInternalData->ConversionParams.HeightScale
				: 1;
		}
		
		if (ConversionParams.ZeroInEncoding != HeightInternalData->ConversionParams.ZeroInEncoding
			|| ConversionParams.HeightScale != HeightInternalData->ConversionParams.HeightScale
			|| ConversionParams.HeightOffset != HeightInternalData->ConversionParams.HeightOffset)
		{
			HeightInternalData->Modify();
			HeightInternalData->ConversionParams = ConversionParams;
		}
	}
#endif
}

void ULandscapeTexturePatch::ResetHeightEncodingMode(ELandscapeTextureHeightPatchEncoding EncodingMode)
{
#if WITH_EDITOR
	Modify();
	HeightEncoding = EncodingMode;
	if (EncodingMode == ELandscapeTextureHeightPatchEncoding::ZeroToOne)
	{
		HeightEncodingSettings.ZeroInEncoding = 0.5;
		HeightEncodingSettings.WorldSpaceEncodingScale = 400;
	}
	else if (EncodingMode == ELandscapeTextureHeightPatchEncoding::WorldUnits)
	{
		HeightEncodingSettings.ZeroInEncoding = 0;
		HeightEncodingSettings.WorldSpaceEncodingScale = 1;
	}
	ResetHeightRenderTargetFormat();

	UpdateHeightConvertToNativeParamsIfNeeded();
#endif
}

#if WITH_EDITOR
void ULandscapeTexturePatch::ResetHeightRenderTargetFormat()
{
	// Heightmap alpha channel is only utilized when SourceTextureChannel (heightmap's alpha) is selected. All other alpha modes generate a second alpha specific texture
	const bool bUsesAlphaChannel = HeightAlphaSettings.GetAlphaSourceMode() == ELandscapeTexturePatchAlphaSourceMode::SourceTextureChannel;
	const ETextureRenderTargetFormat DefaultEncodingFormat = bUsesAlphaChannel ? ETextureRenderTargetFormat::RTF_RGBA32f : ETextureRenderTargetFormat::RTF_R32f;

	const ETextureRenderTargetFormat NewFormat = (HeightEncoding == ELandscapeTextureHeightPatchEncoding::NativePackedHeight) 
		? ETextureRenderTargetFormat::RTF_RGBA8
		: DefaultEncodingFormat;

	SetHeightRenderTargetFormat(NewFormat);

	// When the height uses internal texture / native packed height we limit the user to RG texture color channels
	if (HeightEncoding == ELandscapeTextureHeightPatchEncoding::NativePackedHeight || HeightSourceMode == ELandscapeTexturePatchSourceMode::InternalTexture)
	{
		HeightTextureChannel = ELandscapeTexturePatchTextureChannel::None;
	}
}
#endif

void ULandscapeTexturePatch::SetHeightEncodingSettings(const FLandscapeTexturePatchEncodingSettings& Settings)
{
	Modify();
	HeightEncodingSettings = Settings;

	UpdateHeightConvertToNativeParamsIfNeeded();
}

void ULandscapeTexturePatch::SetHeightRenderTargetFormat(ETextureRenderTargetFormat Format)
{
	if (HeightRenderTargetFormat == Format)
	{
		return;
	}

	Modify();
	HeightRenderTargetFormat = Format;
	if (HeightInternalData)
	{
		HeightInternalData->SetFormat(HeightRenderTargetFormat);
	}
}

// Deprecated
void ULandscapeTexturePatch::AddWeightPatch(const FName& WeightmapLayerName, ELandscapeTexturePatchSourceMode SourceMode, bool bUseAlphaChannel)
{
	CreateWeightPatch(WeightmapLayerName, SourceMode, bUseAlphaChannel ? ELandscapeTexturePatchAlphaSourceMode::SourceTextureChannel : ELandscapeTexturePatchAlphaSourceMode::None);
}

void ULandscapeTexturePatch::CreateWeightPatch(const FName& InWeightmapLayerName, ELandscapeTexturePatchSourceMode InSourceMode, ELandscapeTexturePatchAlphaSourceMode InAlphaSourceMode)
{
#if WITH_EDITOR
	using namespace LandscapeTexturePatchLocals;

	// Try to modify an existing entry instead if possible
	for (const TObjectPtr<ULandscapeWeightPatchTextureInfo>& WeightPatch : WeightPatches)
	{
		if (!IsValid(WeightPatch))
		{
			continue;
		}

		if (WeightPatch->WeightmapLayerName == InWeightmapLayerName)
		{
			if (WeightPatch->SourceMode != InSourceMode)
			{
				WeightPatch->SetSourceMode(InSourceMode);
			}
			// Change the alpha override if needed. This will call UseAlphaChannel on the source internal data
			const FLandscapeTexturePatchAlphaSettings& AlphaSettings = WeightPatch->WeightPatchAlphaSettings;
			if (AlphaSettings.GetAlphaSourceMode() != InAlphaSourceMode)
			{
				WeightPatch->SetWeightPatchAlphaSourceMode(InAlphaSourceMode, /*bInOverrideAlpha =*/true);
			}
			return;
		}
	}

	// The object creation is modeled after SPropertyEditorEditInline::OnClassPicked, which is how these are created
	// from the detail panel. We probably don't need the archetype check, admittedly, but might as well keep it.
	EObjectFlags NewObjectFlags = GetMaskedFlags(RF_PropagateToSubObjects);
	if (HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{
		NewObjectFlags |= RF_ArchetypeObject;
	}
	ULandscapeWeightPatchTextureInfo* NewWeightPatch = NewObject<ULandscapeWeightPatchTextureInfo>(this, NAME_None, NewObjectFlags);

	NewWeightPatch->WeightmapLayerName = InWeightmapLayerName;
	NewWeightPatch->SourceMode = InSourceMode;
	NewWeightPatch->DetailPanelSourceMode = InSourceMode;

	if (InAlphaSourceMode != ELandscapeTexturePatchAlphaSourceMode::None)
	{
		NewWeightPatch->SetWeightPatchAlphaSourceMode(InAlphaSourceMode, /*bInOverrideAlpha =*/true);
	}

	if (NewWeightPatch->SourceMode == ELandscapeTexturePatchSourceMode::InternalTexture
		|| NewWeightPatch->SourceMode == ELandscapeTexturePatchSourceMode::TextureBackedRenderTarget)
	{
		NewWeightPatch->InternalData = NewObject<ULandscapeWeightTextureBackedRenderTarget>(NewWeightPatch);
		// Only set the transactional flag on that new sub-object if the weight patch is (it can happen that is is made temporarily non-transactional in which case, we don't want
		//  one of its sub-objects to become transactional) : 
		if (NewWeightPatch->GetFlags() & RF_Transactional)
		{
			NewWeightPatch->InternalData->SetFlags(RF_Transactional);
		}
		NewWeightPatch->InternalData->SetResolution(ResolutionX, ResolutionY);
		NewWeightPatch->InternalData->SetUseAlphaChannel(InAlphaSourceMode == ELandscapeTexturePatchAlphaSourceMode::SourceTextureChannel);
		NewWeightPatch->InternalData->Initialize();
	}
	WeightPatches.Add(NewWeightPatch);
#endif // WITH_EDITOR
}

void ULandscapeTexturePatch::RemoveWeightPatch(const FName& InWeightmapLayerName)
{
	WeightPatches.RemoveAll([InWeightmapLayerName](const TObjectPtr<ULandscapeWeightPatchTextureInfo>& InWeightPatch)
	{ 
			return IsValid(InWeightPatch) && InWeightPatch->WeightmapLayerName == InWeightmapLayerName;
	});
}

void ULandscapeTexturePatch::RemoveAllWeightPatches()
{
	WeightPatches.Reset();
}

void ULandscapeTexturePatch::DisableAllWeightPatches()
{
	for (const TObjectPtr<ULandscapeWeightPatchTextureInfo>& WeightPatch : WeightPatches)
	{
		if (IsValid(WeightPatch))
		{
			WeightPatch->SetSourceMode(ELandscapeTexturePatchSourceMode::None);
		}
	}
}

TArray<FName> ULandscapeTexturePatch::GetAllWeightPatchLayerNames()
{
	TArray<FName> Names;
	for (const TObjectPtr<ULandscapeWeightPatchTextureInfo>& WeightPatch : WeightPatches)
	{
		if (IsValid(WeightPatch) && WeightPatch->WeightmapLayerName != NAME_None)
		{
			Names.AddUnique(WeightPatch->WeightmapLayerName);
		}
	}

	return Names;
}

ELandscapeTexturePatchAlphaSourceMode ULandscapeTexturePatch::GetWeightPatchAlphaSourceMode(const FName& InWeightmapLayerName) const
{
	// Get the default weightmap alpha source mode for all weightmap patches
	if (InWeightmapLayerName.IsNone())
	{
		return WeightPatchesAlphaSettings.GetAlphaSourceMode();
	}

	if (TObjectPtr<ULandscapeWeightPatchTextureInfo> WeightPatch = GetWeightPatch(InWeightmapLayerName, ANSI_TO_TCHAR(__FUNCTION__)))
	{
		// Returns the override source mode if and only if the alpha settings are overridden
		return GetWeightPatchAlphaSettingsConst(WeightPatch).GetAlphaSourceMode();
	}
	return WeightPatchesAlphaSettings.GetAlphaSourceMode();
}

// Deprecated
void ULandscapeTexturePatch::SetUseAlphaChannelForWeightPatch(const FName& InWeightmapLayerName, bool bUseAlphaChannel)
{
	SetWeightPatchAlphaSourceMode(InWeightmapLayerName, bUseAlphaChannel ? ELandscapeTexturePatchAlphaSourceMode::SourceTextureChannel : ELandscapeTexturePatchAlphaSourceMode::None);
}

void ULandscapeTexturePatch::SetWeightPatchAlphaSourceMode(const FName& InWeightmapLayerName, ELandscapeTexturePatchAlphaSourceMode InNewSourceMode, bool bInOverrideAlpha, bool bAlwaysMarkDirty)
{
	// Set the default weightmap alpha source mode for all weightmap patches
	if (InWeightmapLayerName.IsNone())
	{
		WeightPatchesAlphaSettings.SetAlphaSourceMode(InNewSourceMode, this);
		return;
	}

	// Set the override weightmap source mode for a specific weight patch
	if (TObjectPtr<ULandscapeWeightPatchTextureInfo> WeightPatch = GetWeightPatch(InWeightmapLayerName, ANSI_TO_TCHAR(__FUNCTION__)))
	{
		WeightPatch->SetWeightPatchAlphaSourceMode(InNewSourceMode, bInOverrideAlpha, bAlwaysMarkDirty);
		return;
	}
}

const UTexture* ULandscapeTexturePatch::GetWeightPatchAlphaTextureAsset(const FName& InWeightmapLayerName) const
{	
	// Get the default weightmap alpha texture asset for all weightmap patches
	if (InWeightmapLayerName.IsNone())
	{
		return WeightPatchesAlphaSettings.AlphaTextureAsset;
	}

	if (TObjectPtr<ULandscapeWeightPatchTextureInfo> WeightPatch = GetWeightPatch(InWeightmapLayerName, ANSI_TO_TCHAR(__FUNCTION__)))
	{
		// Returns the override value if and only if the alpha settings are overridden
		return GetWeightPatchAlphaSettingsConst(WeightPatch).AlphaTextureAsset;
	}

	return WeightPatchesAlphaSettings.AlphaTextureAsset;
}

UTextureRenderTarget2D* ULandscapeTexturePatch::GetWeightPatchAlphaRenderTarget(const FName& InWeightmapLayerName, bool bMarkDirty)
{
	if (IsTemplate())
	{
		return nullptr;
	}

	if (TObjectPtr<ULandscapeWeightPatchTextureInfo> WeightPatch = GetWeightPatch(InWeightmapLayerName, ANSI_TO_TCHAR(__FUNCTION__)))
	{
		if (bMarkDirty)
		{
			MarkPackageDirty();
		}

		// returns the default (shared) weight patch render target or the specific override weight patch render target
		return GetWeightPatchAlphaRenderTarget(WeightPatch, bMarkDirty);
	}
	return nullptr;
}

void ULandscapeTexturePatch::SetWeightPatchAlphaTextureAsset(const FName& InWeightmapLayerName, UTexture* InTexture, bool bInOverrideAlpha, bool bAlwaysMarkDirty)
{
	if (InTexture && InTexture->VirtualTextureStreaming != 0)
	{
		UE_LOGF(LogLandscapePatch, Error, "ULandscapeTexturePatch::SetWeightPatchAlphaTextureAsset: Invalid texture. Virtual textures are not supported. Weightmap Layer Name %ls.", *InWeightmapLayerName.ToString());
		return;
	}

	// Set the default weightmap alpha texture for all weightmap patches
	if (InWeightmapLayerName.IsNone())
	{
		if (WeightPatchesAlphaSettings.AlphaTextureAsset == InTexture)
		{
			return;
		}
		Modify(bAlwaysMarkDirty);
		WeightPatchesAlphaSettings.AlphaTextureAsset = InTexture;
		return;
	}

	// Set the override weightmap texture for a specific weight patch
	if (TObjectPtr<ULandscapeWeightPatchTextureInfo> WeightPatch = GetWeightPatch(InWeightmapLayerName, ANSI_TO_TCHAR(__FUNCTION__)))
	{
		WeightPatch->bOverrideWeightAlphaSettings = bInOverrideAlpha;
		// When bInOverrideAlpha is false the weightmap layer will fallback to the shared weightmap alpha
		if (bInOverrideAlpha)
		{
			WeightPatch->WeightPatchAlphaSettings.AlphaTextureAsset = InTexture;
		}
		else
		{
			WeightPatch->SetWeightPatchAlphaSourceMode(ELandscapeTexturePatchAlphaSourceMode::None, /*bInOverrideAlpha =*/false, bAlwaysMarkDirty);
		}
		return;
	}
}

const FLandscapeTexturePatchTextureTransform& ULandscapeTexturePatch::GetWeightPatchAlphaTextureTransform(const FName& InWeightmapLayerName) const
{
	// Get the default weightmap alpha texture transform for all weightmap patches
	if (InWeightmapLayerName.IsNone())
	{
		return WeightPatchesAlphaSettings.AlphaTextureUVTransform;
	}

	if (TObjectPtr<ULandscapeWeightPatchTextureInfo> WeightPatch = GetWeightPatch(InWeightmapLayerName, ANSI_TO_TCHAR(__FUNCTION__)))
	{
		return GetWeightPatchAlphaSettingsConst(WeightPatch).AlphaTextureUVTransform;
	}
	return WeightPatchesAlphaSettings.AlphaTextureUVTransform;
}

void ULandscapeTexturePatch::SetWeightPatchAlphaTextureTransform(const FName& InWeightmapLayerName, const FLandscapeTexturePatchTextureTransform& InTextureTransform, bool bAlwaysMarkDirty)
{
	// Set the default weightmap alpha texture transform for all weightmap patches
	if (InWeightmapLayerName.IsNone())
	{
		if (WeightPatchesAlphaSettings.AlphaTextureUVTransform == InTextureTransform)
		{
			return;
		}
		Modify(bAlwaysMarkDirty);
		WeightPatchesAlphaSettings.AlphaTextureUVTransform = InTextureTransform;
		return;
	}

	if (TObjectPtr<ULandscapeWeightPatchTextureInfo> WeightPatch = GetWeightPatch(InWeightmapLayerName, ANSI_TO_TCHAR(__FUNCTION__)))
	{
		WeightPatch->SetWeightPatchAlphaTextureTransform(InTextureTransform, bAlwaysMarkDirty);
		return;
	}
}

ELandscapeTexturePatchTextureChannel ULandscapeTexturePatch::GetWeightPatchAlphaTextureChannel(const FName& InWeightmapLayerName) const
{
	if (InWeightmapLayerName.IsNone())
	{
		return WeightPatchesAlphaSettings.AlphaTextureChannel;
	}

	if (TObjectPtr<ULandscapeWeightPatchTextureInfo> WeightPatch = GetWeightPatch(InWeightmapLayerName, ANSI_TO_TCHAR(__FUNCTION__)))
	{
		// Returns the override value if and only if the alpha settings are overridden
		return GetWeightPatchAlphaSettingsConst(WeightPatch).AlphaTextureChannel;
	}

	return WeightPatchesAlphaSettings.AlphaTextureChannel;
}

void ULandscapeTexturePatch::SetWeightPatchAlphaTextureChannel(const FName& InWeightmapLayerName, ELandscapeTexturePatchTextureChannel InTextureChannel, bool bInOverrideAlpha, bool bAlwaysMarkDirty)
{
	// Add to the default weightmap alpha channel for all weightmap patches
	if (InWeightmapLayerName.IsNone())
	{
		if (WeightPatchesAlphaSettings.AlphaTextureChannel == InTextureChannel)
		{
			return;
		}
		Modify(bAlwaysMarkDirty);
		WeightPatchesAlphaSettings.AlphaTextureChannel = InTextureChannel;
		return;
	}

	if (TObjectPtr<ULandscapeWeightPatchTextureInfo> WeightPatch = GetWeightPatch(InWeightmapLayerName, ANSI_TO_TCHAR(__FUNCTION__)))
	{
		WeightPatch->bOverrideWeightAlphaSettings = bInOverrideAlpha;
		// When bInOverrideAlpha is false the weightmap layer will fallback to the shared weightmap channel
		if (bInOverrideAlpha)
		{
			WeightPatch->SetWeightPatchAlphaTextureChannel(InTextureChannel, bAlwaysMarkDirty);
		}
		else
		{
			WeightPatch->SetWeightPatchAlphaSourceMode(ELandscapeTexturePatchAlphaSourceMode::None, /*bInOverrideAlpha =*/false, bAlwaysMarkDirty);
		}
		return;
	}
}

void ULandscapeTexturePatch::SetWeightPatchSourceMode(const FName& InWeightmapLayerName, ELandscapeTexturePatchSourceMode NewMode)
{
#if WITH_EDITOR
	using namespace LandscapeTexturePatchLocals;

	if (TObjectPtr<ULandscapeWeightPatchTextureInfo> WeightPatch = GetWeightPatch(InWeightmapLayerName, ANSI_TO_TCHAR(__FUNCTION__)))
	{
		WeightPatch->SetSourceMode(NewMode);
		return;
	}
#endif // WITH_EDITOR
}

ELandscapeTexturePatchSourceMode ULandscapeTexturePatch::GetWeightPatchSourceMode(const FName& InWeightmapLayerName)
{
	if (TObjectPtr<ULandscapeWeightPatchTextureInfo> WeightPatch = GetWeightPatch(InWeightmapLayerName, ANSI_TO_TCHAR(__FUNCTION__)))
	{
		return WeightPatch->SourceMode;
	}
	return ELandscapeTexturePatchSourceMode::None;
}

const FLandscapeTexturePatchTextureTransform& ULandscapeTexturePatch::GetWeightPatchTextureTransform(const FName& InWeightmapLayerName) const
{
	if (TObjectPtr<ULandscapeWeightPatchTextureInfo> WeightPatch = GetWeightPatch(InWeightmapLayerName, ANSI_TO_TCHAR(__FUNCTION__)))
	{
		return WeightPatch->WeightTextureUVTransform;
	}

	// There is no default texture transform for all weight patches (Only the alpha texture is overridable). Log an explicit error here
	UE_LOGF(LogLandscapePatch, Error, "ULandscapeTexturePatch::GetWeightPatchTextureTransform: Unable to get transform data for weight layer %ls. The weight patch transform will fallback to the height texture patch transform.", *InWeightmapLayerName.ToString());
	return HeightTextureUVTransform;
}

void ULandscapeTexturePatch::SetWeightPatchTextureTransform(const FName& InWeightmapLayerName, const FLandscapeTexturePatchTextureTransform& InTextureTransform, bool bAlwaysMarkDirty)
{
	if (TObjectPtr<ULandscapeWeightPatchTextureInfo> WeightPatch = GetWeightPatch(InWeightmapLayerName, ANSI_TO_TCHAR(__FUNCTION__)))
	{
		WeightPatch->SetWeightPatchTextureTransform(InTextureTransform, bAlwaysMarkDirty);
		return;
	}
}

ELandscapeTexturePatchTextureChannel ULandscapeTexturePatch::GetWeightPatchTextureChannel(const FName& InWeightmapLayerName) const
{
	if (TObjectPtr<ULandscapeWeightPatchTextureInfo> WeightPatch = GetWeightPatch(InWeightmapLayerName, ANSI_TO_TCHAR(__FUNCTION__)))
	{
		return WeightPatch->WeightPatchTextureChannel;
	}
	return ELandscapeTexturePatchTextureChannel::None;
}

void ULandscapeTexturePatch::SetWeightPatchTextureChannel(const FName& InWeightmapLayerName, ELandscapeTexturePatchTextureChannel InTextureChannel, bool bAlwaysMarkDirty)
{
	if (TObjectPtr<ULandscapeWeightPatchTextureInfo> WeightPatch = GetWeightPatch(InWeightmapLayerName, ANSI_TO_TCHAR(__FUNCTION__)))
	{
		WeightPatch->SetWeightPatchTextureChannel(InTextureChannel, bAlwaysMarkDirty);
		return;
	}
}

UTexture* ULandscapeTexturePatch::GetWeightPatchTextureAsset(const FName& InWeightmapLayerName)
{
	if (TObjectPtr<ULandscapeWeightPatchTextureInfo> WeightPatch = GetWeightPatch(InWeightmapLayerName, ANSI_TO_TCHAR(__FUNCTION__)))
	{
		return WeightPatch->TextureAsset;
	}
	return nullptr;
}

UTextureRenderTarget2D* ULandscapeTexturePatch::GetWeightPatchRenderTarget(const FName& InWeightmapLayerName, bool bMarkDirty)
{
	if (IsTemplate())
	{
		return nullptr;
	}

	if (TObjectPtr<ULandscapeWeightPatchTextureInfo> WeightPatch = GetWeightPatch(InWeightmapLayerName, ANSI_TO_TCHAR(__FUNCTION__)))
	{
		if (bMarkDirty)
		{
			MarkPackageDirty();
		}

		return GetWeightPatchRenderTarget(WeightPatch);
	}
	return nullptr;
}

UTextureRenderTarget2D* ULandscapeTexturePatch::GetWeightPatchRenderTarget(ULandscapeWeightPatchTextureInfo* WeightPatch)
{
#if WITH_EDITOR
	if (IsTemplate() || !IsValid(WeightPatch))
	{
		return nullptr;
	}

	// Allocate data if needed (see comment in GetHeightRenderTarget)
	if (WeightPatch->SourceMode == ELandscapeTexturePatchSourceMode::TextureBackedRenderTarget)
	{
		if (!WeightPatch->InternalData || !WeightPatch->InternalData->GetRenderTarget())
		{
			WeightPatch->TransitionSourceModeInternal(ELandscapeTexturePatchSourceMode::None, WeightPatch->SourceMode);
		}

		return ensure(WeightPatch->InternalData) ? WeightPatch->InternalData->GetRenderTarget() : nullptr;
	}
#endif

	return nullptr;
}

FLandscapeTexturePatchAlphaSettings& ULandscapeTexturePatch::GetWeightPatchAlphaSettings(ULandscapeWeightPatchTextureInfo* WeightPatch)
{
	if (WeightPatch == nullptr)
	{
		return WeightPatchesAlphaSettings;
	}
	return WeightPatch->bOverrideWeightAlphaSettings ? WeightPatch->WeightPatchAlphaSettings : WeightPatchesAlphaSettings;
}

const FLandscapeTexturePatchAlphaSettings& ULandscapeTexturePatch::GetWeightPatchAlphaSettingsConst(ULandscapeWeightPatchTextureInfo* WeightPatch) const
{
	if (WeightPatch == nullptr)
	{
		return WeightPatchesAlphaSettings;
	}
	return WeightPatch->bOverrideWeightAlphaSettings ? WeightPatch->WeightPatchAlphaSettings : WeightPatchesAlphaSettings;
}


TObjectPtr<ULandscapeWeightPatchTextureInfo> ULandscapeTexturePatch::GetWeightPatch(const FName& InWeightmapLayerName, const TCHAR* InDebugLogName) const
{
	TArray<TObjectPtr<ULandscapeWeightPatchTextureInfo>> FoundPatches = WeightPatches.FilterByPredicate([&InWeightmapLayerName](const TObjectPtr<ULandscapeWeightPatchTextureInfo>& WeightPatch)
		{
			return IsValid(WeightPatch) && WeightPatch->WeightmapLayerName == InWeightmapLayerName;
		});

	const FString LayerNameString = InWeightmapLayerName.ToString();
	const TCHAR* DebugLogName = (InDebugLogName == nullptr) ? TEXT("ULandscapeTexturePatch::GetWeightPatch") : InDebugLogName;

	if (FoundPatches.IsEmpty())
	{
		UE_LOGF(LogLandscapePatch, Warning, "%ls: Unable to find weightmap patch for weightmap layer name %ls", DebugLogName, *LayerNameString);
		return nullptr;
	}

	if (FoundPatches.Num() > 1)
	{
		UE_LOGF(LogLandscapePatch, Warning, "%ls found more than one patch using the same weightmap layer name %ls. BP functions will return the first matching weight patch only!", DebugLogName, *LayerNameString);
	}

	return FoundPatches[0];
}

UTextureRenderTarget2D* ULandscapeTexturePatch::GetWeightPatchAlphaRenderTarget(ULandscapeWeightPatchTextureInfo* WeightPatch, bool bMarkDirty)
{
	if (IsTemplate() || !IsValid(WeightPatch))
	{
		return nullptr;
	}

	if (WeightPatch->bOverrideWeightAlphaSettings)
	{
		return WeightPatch->WeightPatchAlphaSettings.GetAlphaRenderTarget(WeightPatch, bMarkDirty);
	}
	else
	{
		return WeightPatchesAlphaSettings.GetAlphaRenderTarget(this, bMarkDirty);
	}
}

UTexture2D* ULandscapeTexturePatch::GetWeightPatchAlphaInternalTexture(ULandscapeWeightPatchTextureInfo* WeightPatch, bool bMarkDirty)
{
	if (IsTemplate() || !IsValid(WeightPatch))
	{
		return nullptr;
	}

	if (WeightPatch->bOverrideWeightAlphaSettings)
	{
		return WeightPatch->WeightPatchAlphaSettings.GetAlphaInternalTexture(WeightPatch, bMarkDirty);
	}
	else
	{
		return WeightPatchesAlphaSettings.GetAlphaInternalTexture(this, bMarkDirty);
	}
}

UTexture2D* ULandscapeTexturePatch::GetWeightPatchInternalTexture(ULandscapeWeightPatchTextureInfo* WeightPatch)
{
#if WITH_EDITOR
	if (IsTemplate() || !IsValid(WeightPatch))
	{
		return nullptr;
	}

	// Allocate data if needed (see comment in GetHeightRenderTarget)
	if (WeightPatch->SourceMode == ELandscapeTexturePatchSourceMode::InternalTexture 
		|| WeightPatch->SourceMode == ELandscapeTexturePatchSourceMode::TextureBackedRenderTarget)
	{
		if (!WeightPatch->InternalData || !WeightPatch->InternalData->GetInternalTexture())
		{
			WeightPatch->TransitionSourceModeInternal(ELandscapeTexturePatchSourceMode::None, WeightPatch->SourceMode);
		}

		return ensure(WeightPatch->InternalData) ? WeightPatch->InternalData->GetInternalTexture() : nullptr;
	}
#endif

	return nullptr;
}

void ULandscapeTexturePatch::SetWeightPatchTextureAsset(const FName& InWeightmapLayerName, UTexture* TextureIn)
{
	if (TextureIn && TextureIn->VirtualTextureStreaming != 0)
	{
		UE_LOGF(LogLandscapePatch, Error, "ULandscapeTexturePatch::SetWeightPatchTextureAsset: Invalid texture. Virtual textures are not supported. Weightmap Layer Name %ls.", *InWeightmapLayerName.ToString());
		return;
	}

	if (TObjectPtr<ULandscapeWeightPatchTextureInfo> WeightPatch = GetWeightPatch(InWeightmapLayerName, ANSI_TO_TCHAR(__FUNCTION__)))
	{
		WeightPatch->TextureAsset = TextureIn;
		return;
	}

	const FString LayerNameString = InWeightmapLayerName.ToString();
	UE_LOGF(LogLandscapePatch, Warning, "ULandscapeTexturePatch::SetWeightPatchTextureAsset: Unable to find data for weight layer %ls", *LayerNameString);
}

void ULandscapeTexturePatch::SetWeightPatchBlendModeOverride(const FName& InWeightmapLayerName, ELandscapeTexturePatchBlendMode BlendModeIn)
{
	if (TObjectPtr<ULandscapeWeightPatchTextureInfo> WeightPatch = GetWeightPatch(InWeightmapLayerName, ANSI_TO_TCHAR(__FUNCTION__)))
	{
		WeightPatch->OverrideBlendMode = BlendModeIn;
		WeightPatch->bOverrideBlendMode = true;
	}
}

void ULandscapeTexturePatch::ClearWeightPatchBlendModeOverride(const FName& InWeightmapLayerName)
{
	if (TObjectPtr<ULandscapeWeightPatchTextureInfo> WeightPatch = GetWeightPatch(InWeightmapLayerName, ANSI_TO_TCHAR(__FUNCTION__)))
	{
		WeightPatch->bOverrideBlendMode = false;
	}
}

void ULandscapeTexturePatch::SetEditVisibilityLayer(const FName& InWeightmapLayerName, const bool bEditVisibilityLayer)
{
	if (TObjectPtr<ULandscapeWeightPatchTextureInfo> WeightPatch = GetWeightPatch(InWeightmapLayerName, ANSI_TO_TCHAR(__FUNCTION__)))
	{
		WeightPatch->bEditVisibilityLayer = bEditVisibilityLayer;
	}
}

#if WITH_EDITOR
void ULandscapeTexturePatch::GetRendererStateInfo(const UE::Landscape::EditLayers::FMergeContext* InMergeContext,
	UE::Landscape::EditLayers::FEditLayerTargetTypeState& OutSupported, UE::Landscape::EditLayers::FEditLayerTargetTypeState& OutEnabled,
	TArray<UE::Landscape::EditLayers::FTargetLayerGroup>& OutTargetLayerGroups) const
{
	if (InMergeContext->IsHeightmapMerge())
	{
		if (CanAffectHeightmap())
		{
			OutSupported.AddTargetType(ELandscapeToolTargetType::Heightmap);
		}
	}
	else
	{
		for (const TObjectPtr<ULandscapeWeightPatchTextureInfo>& WeightPatch : WeightPatches)
		{
			if (IsValid(WeightPatch) && WeightPatchCanRender(*WeightPatch))
			{
				if (WeightPatch->bEditVisibilityLayer)
				{
					OutSupported.AddTargetType(ELandscapeToolTargetType::Visibility);
				}
				else if (InMergeContext->IsValidTargetLayerName(WeightPatch->WeightmapLayerName))
				{
					const int32 TargetLayerIndex = InMergeContext->GetTargetLayerIndexForNameChecked(WeightPatch->WeightmapLayerName);
					OutSupported.AddTargetType(ELandscapeToolTargetType::Weightmap);
					OutSupported.AddWeightmap(TargetLayerIndex);
				}
			}
		}

		// Build the target layer groups for this step. Since we use the generic blend, we can use BuildGenericBlendTargetLayerGroups
		OutTargetLayerGroups = InMergeContext->BuildGenericBlendTargetLayerGroups(OutSupported.GetActiveWeightmapBitIndices());
	}

	if (IsEnabled())
	{
		OutEnabled = OutSupported;
	}
}

FString ULandscapeTexturePatch::GetEditLayerRendererDebugName() const
{
	return FString::Printf(TEXT("%s:%s"), *GetOwner()->GetActorNameOrLabel(), *GetName());
}

TArray<UE::Landscape::EditLayers::FEditLayerRenderItem> ULandscapeTexturePatch::GetRenderItems(const UE::Landscape::EditLayers::FMergeContext* InMergeContext) const
{
	using namespace UE::Landscape::EditLayers;

	TArray<FEditLayerRenderItem> AffectedAreas;

	FTransform ComponentTransform = this->GetComponentToWorld();
	FOOBox2D PatchArea(ComponentTransform, GetUnscaledCoverage());
	// The output is exactly the patch's area (i.e. object-oriented box)
	FOutputWorldArea OutputWorldArea = FOutputWorldArea::CreateOOBox(PatchArea);
	// Each pixel only depends on the pixel above so we don't need to read anything else than the component itself : 
	FInputWorldArea InputWorldArea = FInputWorldArea::CreateLocalComponent();

	// HACK [jonathan.bard] (the whole reinitialize height/weight is a hack currently anyway : this will disappear once this is implemented via a batched merge partial render) 
	//  When reinitializing height/weight, we need to make sure the patch will be rendered in one batch and one only, because bReinitializeHeightOnNextRender will be reset 
	//  upon rendering, so if 2 batches render the patch, it will be reset between the 2 renders, which will screw up the render command recorder. On top of it, we need to make
	//  sure the entire patch area is rendered in one operation, because we read the result back into a render target
	bool bForceSingleBatch = false;

	if (InMergeContext->IsHeightmapMerge())
	{
		if (CanAffectHeightmap())
		{
			if (bReinitializeHeightOnNextRender)
			{
				bForceSingleBatch = true;
			}

			FEditLayerTargetTypeState TargetInfo(InMergeContext, ELandscapeToolTargetTypeFlags::Heightmap);
			FEditLayerRenderItem Item(TargetInfo, InputWorldArea, OutputWorldArea, false);
			AffectedAreas.Add(Item);
		}
	}
	else
	{
		for (const TObjectPtr<ULandscapeWeightPatchTextureInfo>& WeightPatch : WeightPatches)
		{
			if (IsValid(WeightPatch) && WeightPatchCanRender(*WeightPatch))
			{
				if (WeightPatch->bReinitializeOnNextRender)
				{
					bForceSingleBatch = true;
				}

				FEditLayerTargetTypeState TargetInfo(InMergeContext);
				if (WeightPatch->bEditVisibilityLayer)
				{
					TargetInfo.AddTargetType(ELandscapeToolTargetType::Visibility);
				}
				else if (InMergeContext->IsValidTargetLayerName(WeightPatch->WeightmapLayerName))
				{
					const int32 TargetLayerIndex = InMergeContext->GetTargetLayerIndexForNameChecked(WeightPatch->WeightmapLayerName);
					TargetInfo.AddTargetType(ELandscapeToolTargetType::Weightmap);
					TargetInfo.AddWeightmap(TargetLayerIndex);
				}
				FEditLayerRenderItem Item(TargetInfo, InputWorldArea, OutputWorldArea, /*bInModifyExistingWeightmapsOnly = */false);
				AffectedAreas.Add(Item);
			}
		}
	}

	// TODO [jonathan.bard] Remove once landscape edit layers partial merge replaces bReinitializeHeightOnNextRender / bReinitializeOnNextRender
	if (bForceSingleBatch)
	{
		Algo::ForEach(AffectedAreas, [](FEditLayerRenderItem& InRenderItem) 
			{
				InRenderItem.SetInputWorldArea(FInputWorldArea::CreateInfinite());
			});
	}

	return AffectedAreas;
}
#endif // WITH_EDITOR

#undef LOCTEXT_NAMESPACE
