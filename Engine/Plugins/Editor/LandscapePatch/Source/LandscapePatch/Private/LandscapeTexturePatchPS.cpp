// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeTexturePatchPS.h"

#include "LandscapeUtils.h"
#include "LandscapePatchUtil.h"
#include "PixelShaderUtils.h"

namespace UE::Landscape
{

bool FApplyLandscapeTextureHeightPatchPS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return DoesPlatformSupportEditLayers(Parameters.Platform);
}

void FApplyLandscapeTextureHeightPatchPS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	UE::Landscape::ModifyShaderCompilerEnvironmentForDebug(OutEnvironment);

	OutEnvironment.SetDefine(TEXT("APPLY_HEIGHT_PATCH"), 1);

	// Make our flag choices match in the shader.
	OutEnvironment.SetDefine(TEXT("RECTANGULAR_FALLOFF_FLAG"), static_cast<uint8>(FApplyLandscapeTextureHeightPatchPS::EFlags::RectangularFalloff));
	OutEnvironment.SetDefine(TEXT("INPUT_IS_PACKED_HEIGHT_FLAG"), static_cast<uint8>(FApplyLandscapeTextureHeightPatchPS::EFlags::InputIsPackedHeight));

	OutEnvironment.SetDefine(TEXT("ALPHA_BLEND_MODE"), static_cast<uint8>(ELandscapeTexturePatchBlendMode::AlphaBlend));
	OutEnvironment.SetDefine(TEXT("ADDITIVE_MODE"), static_cast<uint8>(ELandscapeTexturePatchBlendMode::Additive));
	OutEnvironment.SetDefine(TEXT("MIN_MODE"), static_cast<uint8>(ELandscapeTexturePatchBlendMode::Min));
	OutEnvironment.SetDefine(TEXT("MAX_MODE"), static_cast<uint8>(ELandscapeTexturePatchBlendMode::Max));

	OutEnvironment.SetDefine(TEXT("ALPHA_PATCH_NONE"), static_cast<uint8>(ELandscapeTexturePatchAlphaSourceMode::None));
	OutEnvironment.SetDefine(TEXT("ALPHA_PATCH_INTERNAL_TEXTURE"), static_cast<uint8>(ELandscapeTexturePatchAlphaSourceMode::InternalTexture));
	OutEnvironment.SetDefine(TEXT("ALPHA_PATCH_RENDER_TARGET"), static_cast<uint8>(ELandscapeTexturePatchAlphaSourceMode::TextureBackedRenderTarget));
	OutEnvironment.SetDefine(TEXT("ALPHA_PATCH_TEXTURE_ASSET"), static_cast<uint8>(ELandscapeTexturePatchAlphaSourceMode::TextureAsset));
	OutEnvironment.SetDefine(TEXT("ALPHA_PATCH_SRC_TEXTURE_CHANNEL"), static_cast<uint8>(ELandscapeTexturePatchAlphaSourceMode::SourceTextureChannel));

	OutEnvironment.SetDefine(TEXT("TEXTURE_CHANNEL_R"), static_cast<uint8>(ELandscapeTexturePatchTextureChannel::Red));
	OutEnvironment.SetDefine(TEXT("TEXTURE_CHANNEL_G"), static_cast<uint8>(ELandscapeTexturePatchTextureChannel::Green));
	OutEnvironment.SetDefine(TEXT("TEXTURE_CHANNEL_B"), static_cast<uint8>(ELandscapeTexturePatchTextureChannel::Blue));
	OutEnvironment.SetDefine(TEXT("TEXTURE_CHANNEL_A"), static_cast<uint8>(ELandscapeTexturePatchTextureChannel::Alpha));
	OutEnvironment.SetDefine(TEXT("TEXTURE_CHANNEL_LUM"), static_cast<uint8>(ELandscapeTexturePatchTextureChannel::Luminance));
}

void FApplyLandscapeTextureHeightPatchPS::AddToRenderGraph(FRDGEventName&& InRDGEventName, FRDGBuilder& GraphBuilder, FParameters* InParameters, const FIntRect& DestinationBounds)
{
	FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
	TShaderMapRef<FApplyLandscapeTextureHeightPatchPS> PixelShader(ShaderMap);

	FPixelShaderUtils::AddFullscreenPass(
		GraphBuilder,
		ShaderMap,
		MoveTemp(InRDGEventName),
		PixelShader,
		InParameters,
		DestinationBounds);
}

bool FOffsetHeightmapPS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return DoesPlatformSupportEditLayers(Parameters.Platform);
}

void FOffsetHeightmapPS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	UE::Landscape::ModifyShaderCompilerEnvironmentForDebug(OutEnvironment);

	OutEnvironment.SetDefine(TEXT("OFFSET_HEIGHT_PATCH"), 1);
}

void FOffsetHeightmapPS::AddToRenderGraph(FRDGBuilder& GraphBuilder, FParameters* InParameters)
{
	FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
	TShaderMapRef<FOffsetHeightmapPS> PixelShader(ShaderMap);

	FIntVector TextureSize = InParameters->InHeightmap->Desc.Texture->Desc.GetSize();

	FPixelShaderUtils::AddFullscreenPass(
		GraphBuilder,
		ShaderMap,
		RDG_EVENT_NAME("OffsetHeightmap"),
		PixelShader,
		InParameters,
		FIntRect(0, 0, TextureSize.X, TextureSize.Y));
}

bool FSimpleTextureCopyPS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return DoesPlatformSupportEditLayers(Parameters.Platform);
}

void FSimpleTextureCopyPS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	UE::Landscape::ModifyShaderCompilerEnvironmentForDebug(OutEnvironment);

	OutEnvironment.SetDefine(TEXT("SIMPLE_TEXTURE_COPY"), 1);
}

void FSimpleTextureCopyPS::AddToRenderGraph(FRDGBuilder& GraphBuilder, FRDGTextureRef SourceTexture, FRDGTextureRef DestinationTexture)
{
	FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
	TShaderMapRef<FSimpleTextureCopyPS> PixelShader(ShaderMap);

	FRDGTextureSRVRef SourceSRV = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(SourceTexture, 0));

	FSimpleTextureCopyPS::FParameters* ShaderParams = GraphBuilder.AllocParameters<FSimpleTextureCopyPS::FParameters>();
	ShaderParams->InSource = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(SourceTexture, 0));
	ShaderParams->InSourceSampler = TStaticSamplerState<SF_Bilinear, AM_Wrap, AM_Wrap, AM_Wrap>::GetRHI();
	ShaderParams->RenderTargets[0] = FRenderTargetBinding(DestinationTexture, ERenderTargetLoadAction::ENoAction, /*InMipIndex = */0);
	FIntVector DestinationSize = DestinationTexture->Desc.GetSize();
	ShaderParams->InDestinationResolution = FVector2f(DestinationSize.X, DestinationSize.Y);

	FPixelShaderUtils::AddFullscreenPass(
		GraphBuilder,
		ShaderMap,
		RDG_EVENT_NAME("SimpleTextureCopy"),
		PixelShader,
		ShaderParams,
		FIntRect(0, 0, DestinationSize.X, DestinationSize.Y));
}

bool FConvertToNativeLandscapePatchPS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return DoesPlatformSupportEditLayers(Parameters.Platform);
}

void FConvertToNativeLandscapePatchPS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	UE::Landscape::ModifyShaderCompilerEnvironmentForDebug(OutEnvironment);

	OutEnvironment.SetDefine(TEXT("CONVERT_TO_NATIVE_LANDSCAPE_PATCH"), 1);
}

void FConvertToNativeLandscapePatchPS::AddToRenderGraph(FRDGBuilder& GraphBuilder, FRDGTextureRef SourceTexture, 
	FRDGTextureRef DestinationTexture, const FLandscapeHeightPatchConvertToNativeParams& Params)
{
	FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
	TShaderMapRef<FConvertToNativeLandscapePatchPS> PixelShader(ShaderMap);

	FRDGTextureSRVRef SourceSRV = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(SourceTexture, 0));

	FConvertToNativeLandscapePatchPS::FParameters* ShaderParams = GraphBuilder.AllocParameters<FConvertToNativeLandscapePatchPS::FParameters>();
	ShaderParams->InHeightmap = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(SourceTexture, 0));
	ShaderParams->InZeroInEncoding = Params.ZeroInEncoding;
	ShaderParams->InHeightScale = Params.HeightScale;
	ShaderParams->InHeightOffset = Params.HeightOffset;
	ShaderParams->RenderTargets[0] = FRenderTargetBinding(DestinationTexture, ERenderTargetLoadAction::ENoAction, /*InMipIndex = */0);
	FIntVector DestinationSize = DestinationTexture->Desc.GetSize();

	FPixelShaderUtils::AddFullscreenPass(
		GraphBuilder,
		ShaderMap,
		RDG_EVENT_NAME("ConvertToNativeLandscapePatch"),
		PixelShader,
		ShaderParams,
		FIntRect(0, 0, DestinationSize.X, DestinationSize.Y));
}

bool FConvertBackFromNativeLandscapePatchPS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return DoesPlatformSupportEditLayers(Parameters.Platform);
}

void FConvertBackFromNativeLandscapePatchPS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	UE::Landscape::ModifyShaderCompilerEnvironmentForDebug(OutEnvironment);

	OutEnvironment.SetDefine(TEXT("CONVERT_BACK_FROM_NATIVE_LANDSCAPE_PATCH"), 1);
}

void FConvertBackFromNativeLandscapePatchPS::AddToRenderGraph(FRDGBuilder& GraphBuilder, FRDGTextureRef SourceTexture,
	FRDGTextureRef DestinationTexture, const FLandscapeHeightPatchConvertToNativeParams& Params)
{
	FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
	TShaderMapRef<FConvertBackFromNativeLandscapePatchPS> PixelShader(ShaderMap);

	FRDGTextureSRVRef SourceSRV = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(SourceTexture, 0));

	FConvertBackFromNativeLandscapePatchPS::FParameters* ShaderParams = GraphBuilder.AllocParameters<FConvertBackFromNativeLandscapePatchPS::FParameters>();
	ShaderParams->InHeightmap = GraphBuilder.CreateSRV(FRDGTextureSRVDesc::CreateForMipLevel(SourceTexture, 0));
	ShaderParams->InZeroInEncoding = Params.ZeroInEncoding;
	ShaderParams->InHeightScale = Params.HeightScale;
	ShaderParams->InHeightOffset = Params.HeightOffset;
	ShaderParams->RenderTargets[0] = FRenderTargetBinding(DestinationTexture, ERenderTargetLoadAction::ENoAction, /*InMipIndex = */0);
	FIntVector DestinationSize = DestinationTexture->Desc.GetSize();

	FPixelShaderUtils::AddFullscreenPass(
		GraphBuilder,
		ShaderMap,
		RDG_EVENT_NAME("ConvertBackFromNativeLandscapePatch"),
		PixelShader,
		ShaderParams,
		FIntRect(0, 0, DestinationSize.X, DestinationSize.Y));
}

bool FApplyLandscapeTextureWeightPatchPS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return DoesPlatformSupportEditLayers(Parameters.Platform);
}

void FApplyLandscapeTextureWeightPatchPS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	UE::Landscape::ModifyShaderCompilerEnvironmentForDebug(OutEnvironment);

	OutEnvironment.SetDefine(TEXT("APPLY_WEIGHT_PATCH"), 1);

	// Make our flag choices match in the shader.
	OutEnvironment.SetDefine(TEXT("RECTANGULAR_FALLOFF_FLAG"), static_cast<uint8>(FApplyLandscapeTextureWeightPatchPS::EFlags::RectangularFalloff));

	OutEnvironment.SetDefine(TEXT("ALPHA_BLEND_MODE"), static_cast<uint8>(ELandscapeTexturePatchBlendMode::AlphaBlend));
	OutEnvironment.SetDefine(TEXT("ADDITIVE_MODE"), static_cast<uint8>(ELandscapeTexturePatchBlendMode::Additive));
	OutEnvironment.SetDefine(TEXT("MIN_MODE"), static_cast<uint8>(ELandscapeTexturePatchBlendMode::Min));
	OutEnvironment.SetDefine(TEXT("MAX_MODE"), static_cast<uint8>(ELandscapeTexturePatchBlendMode::Max));

	OutEnvironment.SetDefine(TEXT("ALPHA_PATCH_NONE"), static_cast<uint8>(ELandscapeTexturePatchAlphaSourceMode::None));
	OutEnvironment.SetDefine(TEXT("ALPHA_PATCH_INTERNAL_TEXTURE"), static_cast<uint8>(ELandscapeTexturePatchAlphaSourceMode::InternalTexture));
	OutEnvironment.SetDefine(TEXT("ALPHA_PATCH_RENDER_TARGET"), static_cast<uint8>(ELandscapeTexturePatchAlphaSourceMode::TextureBackedRenderTarget));
	OutEnvironment.SetDefine(TEXT("ALPHA_PATCH_TEXTURE_ASSET"), static_cast<uint8>(ELandscapeTexturePatchAlphaSourceMode::TextureAsset));
	OutEnvironment.SetDefine(TEXT("ALPHA_PATCH_SRC_TEXTURE_CHANNEL"), static_cast<uint8>(ELandscapeTexturePatchAlphaSourceMode::SourceTextureChannel));

	OutEnvironment.SetDefine(TEXT("TEXTURE_CHANNEL_R"), static_cast<uint8>(ELandscapeTexturePatchTextureChannel::Red));
	OutEnvironment.SetDefine(TEXT("TEXTURE_CHANNEL_G"), static_cast<uint8>(ELandscapeTexturePatchTextureChannel::Green));
	OutEnvironment.SetDefine(TEXT("TEXTURE_CHANNEL_B"), static_cast<uint8>(ELandscapeTexturePatchTextureChannel::Blue));
	OutEnvironment.SetDefine(TEXT("TEXTURE_CHANNEL_A"), static_cast<uint8>(ELandscapeTexturePatchTextureChannel::Alpha));
	OutEnvironment.SetDefine(TEXT("TEXTURE_CHANNEL_LUM"), static_cast<uint8>(ELandscapeTexturePatchTextureChannel::Luminance));
}

void FApplyLandscapeTextureWeightPatchPS::AddToRenderGraph(FRDGEventName&& InRDGEventName, FRDGBuilder& GraphBuilder, FParameters* InParameters, const FIntRect& DestinationBounds)
{
	FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
	TShaderMapRef<FApplyLandscapeTextureWeightPatchPS> PixelShader(ShaderMap);

	FPixelShaderUtils::AddFullscreenPass(
		GraphBuilder,
		ShaderMap,
		MoveTemp(InRDGEventName),
		PixelShader,
		InParameters,
		DestinationBounds);
}

bool FReinitializeLandscapePatchPS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
{
	return DoesPlatformSupportEditLayers(Parameters.Platform);
}

void FReinitializeLandscapePatchPS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
{
	UE::Landscape::ModifyShaderCompilerEnvironmentForDebug(OutEnvironment);

	OutEnvironment.SetDefine(TEXT("REINITIALIZE_PATCH"), 1);
}

void FReinitializeLandscapePatchPS::AddToRenderGraph(FRDGBuilder& GraphBuilder, FParameters* InParameters, bool bHeightPatch)
{
	FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);

	FReinitializeLandscapePatchPS::FPermutationDomain PermutationDomain;
	PermutationDomain.Set<FReinitializeLandscapePatchPS::FHeightPatch>(bHeightPatch);
	TShaderMapRef<FReinitializeLandscapePatchPS> PixelShader(ShaderMap, PermutationDomain);

	FIntVector DestinationSize = InParameters->RenderTargets[0].GetTexture()->Desc.GetSize();

	FPixelShaderUtils::AddFullscreenPass(
		GraphBuilder,
		ShaderMap,
		RDG_EVENT_NAME("ReinitializeLandscapeTexturePatch"),
		PixelShader,
		InParameters,
		FIntRect(0, 0, DestinationSize.X, DestinationSize.Y));
}

}//end UE::Landscape

IMPLEMENT_GLOBAL_SHADER(UE::Landscape::FApplyLandscapeTextureHeightPatchPS, "/Plugin/LandscapePatch/Private/LandscapeTexturePatchPS.usf", "ApplyLandscapeTextureHeightPatch", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(UE::Landscape::FOffsetHeightmapPS, "/Plugin/LandscapePatch/Private/LandscapeTexturePatchPS.usf", "ApplyOffsetToHeightmap", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(UE::Landscape::FSimpleTextureCopyPS, "/Plugin/LandscapePatch/Private/LandscapeTexturePatchPS.usf", "SimpleTextureCopy", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(UE::Landscape::FConvertToNativeLandscapePatchPS, "/Plugin/LandscapePatch/Private/LandscapeTexturePatchPS.usf", "ConvertToNativeLandscapePatch", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(UE::Landscape::FConvertBackFromNativeLandscapePatchPS, "/Plugin/LandscapePatch/Private/LandscapeTexturePatchPS.usf", "ConvertBackFromNativeLandscapePatch", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(UE::Landscape::FApplyLandscapeTextureWeightPatchPS, "/Plugin/LandscapePatch/Private/LandscapeTexturePatchPS.usf", "ApplyLandscapeTextureWeightPatch", SF_Pixel);
IMPLEMENT_GLOBAL_SHADER(UE::Landscape::FReinitializeLandscapePatchPS, "/Plugin/LandscapePatch/Private/LandscapeTexturePatchPS.usf", "ReinitializePatch", SF_Pixel);
