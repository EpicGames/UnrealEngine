// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "GlobalShader.h"
#include "ShaderParameterStruct.h"
#include "LandscapeTexturePatch.h"


struct FLandscapeHeightPatchConvertToNativeParams;

namespace UE::Landscape
{

	/**
	 * Shader that applies a texture-based height patch to a landscape heightmap.
	 */
	class FApplyLandscapeTextureHeightPatchPS : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FApplyLandscapeTextureHeightPatchPS);
		SHADER_USE_PARAMETER_STRUCT(FApplyLandscapeTextureHeightPatchPS, FGlobalShader);

	public:

		// Flags that get packed into a bitfield because we're not allowed to use bool shader parameters:
		enum class EFlags : uint8
		{
			None = 0,

			// When false, falloff is circular.
			RectangularFalloff = 1 << 0,

			// When false, the input is directly interpreted as being the height value to process. When true, the height
			// is unpacked from the red and green channels to make a 16 bit int.
			InputIsPackedHeight = 1 << 1
		};

		// TODO: We could consider exposing an additional global alpha setting that we can use to pass in the given
		// edit layer alpha value... On the other hand, we currently don't bother doing this in any existing blueprint
		// brushes, and it would be hard to support in a way that doesn't require each blueprint brush to respect it
		// individually... Not clear whether this is something worth doing yet.

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, InHeightPatch)
			SHADER_PARAMETER_SAMPLER(SamplerState, InHeightPatchSampler)
			SHADER_PARAMETER(FMatrix44f, InHeightmapToPatchUV)
			// InHeightmapToPatchUV matrix with the source texture transform (world space or relative) applied
			SHADER_PARAMETER(FMatrix44f, InHeightmapToPatchUVTransform)
			// Value in patch that corresponds to the landscape mid value, which is our "0 height".
			SHADER_PARAMETER(float, InZeroInEncoding)
			// Scale to apply to source values relative to the value that represents 0 height.
			SHADER_PARAMETER(float, InHeightScale)
			// Offset to apply to height result after applying height scale
			SHADER_PARAMETER(float, InHeightOffset)
			// Amount of the patch edge to not apply in UV space. Generally set to 0.5/Dimensions to avoid applying
			// the edge half-pixels.
			SHADER_PARAMETER(FVector2f, InEdgeUVDeadBorder)
			// In world units, the size of the margin across which the alpha falls from 1 to 0
			SHADER_PARAMETER(float, InFalloffWorldMargin)
			// Size of the patch in world units (used for falloff)
			SHADER_PARAMETER(FVector2f, InPatchWorldDimensions)
			SHADER_PARAMETER(uint32, InBlendMode)
			// Some combination of the flags (see constants above).
			SHADER_PARAMETER(uint32, InFlags)
			// Height source texture color channel
			SHADER_PARAMETER(uint32, InTextureChannel)
			// Alpha source mode, override texture (White system texture if undefined), alpha texture color channel, and texture transform
			SHADER_PARAMETER(uint32, InAlphaMode)
			SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, InHeightPatchAlphaTexture)
			SHADER_PARAMETER(uint32, InAlphaTextureChannel)
			// InHeightmapToPatchUV matrix with the alpha source texture transform (world space or relative) applied
			SHADER_PARAMETER(FMatrix44f, InHeightmapAlphaToPatchUVTransform)
			// Assigned edit layer's heightmap alpha range [-1, 1]
			SHADER_PARAMETER(float, InEditLayerHeightmapAlpha)

			RENDER_TARGET_BINDING_SLOTS() // Holds our output
		END_SHADER_PARAMETER_STRUCT()

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);
		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);

		static void AddToRenderGraph(FRDGEventName&& InRDGEventName, FRDGBuilder& GraphBuilder, FParameters* InParameters, const FIntRect& DestinationBounds);
	};

	// We need an alias for FApplyLandscapeTextureHeightPatchPS::FParameters so that it can be forward-declared in another header file (cannot forward-declare internal classes)
	class FApplyLandscapeTextureHeightPatchPSParameters : public FApplyLandscapeTextureHeightPatchPS::FParameters {};

	ENUM_CLASS_FLAGS(FApplyLandscapeTextureHeightPatchPS::EFlags);

	/**
	 * Simple shader that just offsets each height value in a height patch by a constant.
	 */
	class FOffsetHeightmapPS : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FOffsetHeightmapPS);
		SHADER_USE_PARAMETER_STRUCT(FOffsetHeightmapPS, FGlobalShader);

	public:

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, InHeightmap)
			SHADER_PARAMETER(float, InHeightOffset)
			RENDER_TARGET_BINDING_SLOTS() // Holds our output
		END_SHADER_PARAMETER_STRUCT()

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);
		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);

		static void AddToRenderGraph(FRDGBuilder& GraphBuilder, FParameters* InParameters);
	};

	/**
	 * Shader that converts a texture stored in some other encoding (where height is in the R channel) to the
	 * landscape "native" encoding, where height is stored as a 16 bit int split across the R and G channels.
	 * This is not perfectly reversible (in case of clamping and due to rounding), but it lets us store the
	 * texture in the way that it would be applied to the landscape (usually).
	 */
	class FConvertToNativeLandscapePatchPS : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FConvertToNativeLandscapePatchPS);
		SHADER_USE_PARAMETER_STRUCT(FConvertToNativeLandscapePatchPS, FGlobalShader);

	public:

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, InHeightmap)
			SHADER_PARAMETER(float, InZeroInEncoding)
			SHADER_PARAMETER(float, InHeightScale)
			SHADER_PARAMETER(float, InHeightOffset)
			RENDER_TARGET_BINDING_SLOTS() // Holds our output
		END_SHADER_PARAMETER_STRUCT()

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);
		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);

		static void AddToRenderGraph(FRDGBuilder& GraphBuilder, FRDGTextureRef SourceTexture,
			FRDGTextureRef DestinationTexture, const FLandscapeHeightPatchConvertToNativeParams& Params);
	};

	/**
	 * Shader that undoes the conversion done by FConvertToNativeLandscapePatchPS (to the extent possible, since
	 * rounding/clamping makes it not perfectly recoverable).
	 */
	class FConvertBackFromNativeLandscapePatchPS : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FConvertBackFromNativeLandscapePatchPS);
		SHADER_USE_PARAMETER_STRUCT(FConvertBackFromNativeLandscapePatchPS, FGlobalShader);

	public:

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, InHeightmap)
			SHADER_PARAMETER(float, InZeroInEncoding)
			SHADER_PARAMETER(float, InHeightScale)
			SHADER_PARAMETER(float, InHeightOffset)
			RENDER_TARGET_BINDING_SLOTS() // Holds our output
		END_SHADER_PARAMETER_STRUCT()

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);
		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);

		static void AddToRenderGraph(FRDGBuilder& GraphBuilder, FRDGTextureRef SourceTexture,
			FRDGTextureRef DestinationTexture, const FLandscapeHeightPatchConvertToNativeParams& Params);
	};

	class FApplyLandscapeTextureWeightPatchPS : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FApplyLandscapeTextureWeightPatchPS);
		SHADER_USE_PARAMETER_STRUCT(FApplyLandscapeTextureWeightPatchPS, FGlobalShader);

	public:

		// Flags that get packed into a bitfield because we're not allowed to use bool shader parameters:
		enum class EFlags : uint8
		{
			None = 0,

			// When false, falloff is circular.
			RectangularFalloff = 1 << 0,
		};

		// TODO: We could consider exposing an additional global alpha setting that we can use to pass in the given
		// edit layer alpha value... On the other hand, we currently don't bother doing this in any existing blueprint
		// brushes, and it would be hard to support in a way that doesn't require each blueprint brush to respect it
		// individually... Not clear whether this is something worth doing yet.

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, InWeightPatch)
			SHADER_PARAMETER_SAMPLER(SamplerState, InWeightPatchSampler)
			SHADER_PARAMETER(FMatrix44f, InWeightmapToPatchUV)
			// InWeightmapToPatchUV matrix with the source texture transform (world space or relative) applied
			SHADER_PARAMETER(FMatrix44f, InWeightmapToPatchUVTransform)
			// Amount of the patch edge to not apply in UV space. Generally set to 0.5/Dimensions to avoid applying
			// the edge half-pixels.
			SHADER_PARAMETER(FVector2f, InEdgeUVDeadBorder)
			// In world units, the size of the margin across which the alpha falls from 1 to 0
			SHADER_PARAMETER(float, InFalloffWorldMargin)
			// Size of the patch in world units (used for falloff)
			SHADER_PARAMETER(FVector2f, InPatchWorldDimensions)
			SHADER_PARAMETER(uint32, InBlendMode)
			// Some combination of the flags (see constants above).
			SHADER_PARAMETER(uint32, InFlags)
			// Weight source texture color channel
			SHADER_PARAMETER(uint32, InTextureChannel)
			// Alpha source mode, override texture (White system texture if undefined), alpha texture color channel, and texture transform
			SHADER_PARAMETER(uint32, InAlphaMode)
			SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, InWeightPatchAlphaTexture)
			SHADER_PARAMETER(uint32, InAlphaTextureChannel)
			// InWeightmapToPatchUV matrix with the alpha source texture transform (world space or relative) applied
			SHADER_PARAMETER(FMatrix44f, InWeightmapAlphaToPatchUVTransform)
			// Assigned edit layer's weightmap alpha range [0, 1]
			SHADER_PARAMETER(float, InEditLayerWeightmapAlpha)

			RENDER_TARGET_BINDING_SLOTS() // Holds our output
		END_SHADER_PARAMETER_STRUCT()

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);
		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);

		static void AddToRenderGraph(FRDGEventName&& InRDGEventName, FRDGBuilder& GraphBuilder, FParameters* InParameters, const FIntRect& DestinationBounds);
	};

	// We need an alias for FApplyLandscapeTextureWeightPatchPSParameters::FParameters so that it can be forward-declared in another header file (cannot forward-declare internal classes)
	class FApplyLandscapeTextureWeightPatchPSParameters : public FApplyLandscapeTextureWeightPatchPS::FParameters {};

	ENUM_CLASS_FLAGS(FApplyLandscapeTextureWeightPatchPS::EFlags);

	class FReinitializeLandscapePatchPS : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FReinitializeLandscapePatchPS);
		SHADER_USE_PARAMETER_STRUCT(FReinitializeLandscapePatchPS, FGlobalShader);

	public:

		class FHeightPatch : SHADER_PERMUTATION_BOOL("HEIGHT_PATCH");
		using FPermutationDomain = TShaderPermutationDomain<FHeightPatch>;

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, InSource)
			SHADER_PARAMETER_SAMPLER(SamplerState, InSourceSampler)
			SHADER_PARAMETER(FMatrix44f, InPatchToSource)

			RENDER_TARGET_BINDING_SLOTS() // Holds our output
		END_SHADER_PARAMETER_STRUCT()

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);
		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);

		static void AddToRenderGraph(FRDGBuilder& GraphBuilder, FParameters* InParameters, bool bHeightPatch);
	};

	/**
	 * Simple shader for copying textures of potentially different resolutions.
	 */
	 //~ Theoretically CopyToResolveTarget or AddCopyToResolveTargetPass should work, but I was unable to make them
	 //~ work without lots of complaints from the renderer.
	class FSimpleTextureCopyPS : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FSimpleTextureCopyPS);
		SHADER_USE_PARAMETER_STRUCT(FSimpleTextureCopyPS, FGlobalShader);

	public:

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, InSource)
			SHADER_PARAMETER_SAMPLER(SamplerState, InSourceSampler)
			SHADER_PARAMETER(FVector2f, InDestinationResolution)
			RENDER_TARGET_BINDING_SLOTS() // Holds our output
		END_SHADER_PARAMETER_STRUCT()

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);
		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);

		static void AddToRenderGraph(FRDGBuilder& GraphBuilder, FRDGTextureRef SourceTexture, FRDGTextureRef DestinationTexture);
	};

}//end UE::Landscape