// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "LandscapePatchComponent.h"
#include "LandscapeEditTypes.h"
#include "RHIAccess.h"

#include "LandscapeCircleHeightPatch.generated.h"

class FTextureResource;

namespace UE::Landscape
{
	class FRDGBuilderRecorder;
} // namespace namespace UE::Landscape

/**
 * The simplest height patch: a circle of flat ground with a falloff past the initial radius across which the
 * alpha decreases linearly. When added to an actor, initializes itself to the bottom of the bounding box.
 */
UCLASS(Blueprintable, BlueprintType, ClassGroup = Landscape, meta = (BlueprintSpawnableComponent))
class ULandscapeCircleHeightPatch : public ULandscapePatchComponent
{
	GENERATED_BODY()

public:

#if WITH_EDITOR
	// ILandscapeEditLayerRenderer, via ULandscapePatchComponent
	virtual void GetRendererStateInfo(const UE::Landscape::EditLayers::FMergeContext* InMergeContext,
		UE::Landscape::EditLayers::FEditLayerTargetTypeState& OutSupportedTargetTypeState, 
		UE::Landscape::EditLayers::FEditLayerTargetTypeState& OutEnabledTargetTypeState, 
		TArray<UE::Landscape::EditLayers::FTargetLayerGroup>& OutTargetLayerGroups) const override;
	virtual FString GetEditLayerRendererDebugName() const override;
	virtual TArray<UE::Landscape::EditLayers::FEditLayerRenderItem> GetRenderItems(const UE::Landscape::EditLayers::FMergeContext* InMergeContext) const override;
	virtual UE::Landscape::EditLayers::ERenderFlags GetRenderFlags(const UE::Landscape::EditLayers::FMergeContext* InMergeContext) const override;
	virtual bool CanGroupRenderLayerWith(TScriptInterface<ILandscapeEditLayerRenderer> InOtherRenderer) const override;
	virtual bool RenderLayer(UE::Landscape::EditLayers::FRenderParams& RenderParams, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder) override;
	virtual void BlendLayer(UE::Landscape::EditLayers::FRenderParams& RenderParams, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder) override;
#endif

	virtual bool CanAffectHeightmap() const override { return !bEditVisibility; }
	virtual bool CanAffectVisibilityLayer() const override { return bEditVisibility; }

	// UActorComponent
	virtual void OnComponentCreated() override;

protected:

	UPROPERTY(EditAnywhere, Category = Settings)
	float Radius = 500;

	/** Distance across which the alpha will go from 1 down to 0 outside of circle. */
	UPROPERTY(EditAnywhere, Category = Settings)
	float Falloff = 500;

	/** Specifies if this patch edits the visibility layer instead of height. */
	UPROPERTY(EditAnywhere, Category = Settings)
	bool bEditVisibility = false;

	/** When true, only the vertices in the circle have alpha 1. If false, the radius is expanded slightly so that neighboring 
	  vertices are also included and the whole circle is able to lie flat. */
	UPROPERTY(EditAnywhere, Category = Settings, AdvancedDisplay)
	bool bExclusiveRadius = false;

private:
#if WITH_EDITOR
	bool ApplyCirclePatch(UE::Landscape::EditLayers::FRenderParams* RenderParams, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder, bool bIsVisibilityLayer,
		FTextureResource* InMergedLandscapeTextureResource, int32 LandscapeTextureSliceIndex,
		const FIntPoint& DestinationResolution, const FTransform& HeightmapCoordsToWorld, ERHIAccess OutputAccess = ERHIAccess::None);
#endif // WITH_EDITOR
};
