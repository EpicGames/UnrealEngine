// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Components/SceneComponent.h"
#include "LandscapeBlueprintBrushBase.h"
#include "LandscapeEditLayerRenderer.h"
#include "LandscapeEditTypes.h"
#include "LandscapePatchEditLayer.h" // PATCH_PRIORITY_BASE
#include "UObject/WeakInterfacePtr.h"

#include "LandscapePatchComponent.generated.h"

enum class ECacheApplyPhase;
enum class ETeleportType : uint8;

class ALandscape;
class ADEPRECATED_LandscapePatchManager;
class UTextureRenderTarget2D;

UENUM()
enum class ELandscapePatchPriorityInitialization : uint8
{
	// Initialize priority to highest currently known value, so that the new patch is on top of any existing
	// patches. Note that the highest known priority could be out of date in between landscape updates if
	// priorities change, so it is possible that adjustment will still be needed.
	AcquireHighest,

	// Do not change the default/archetype priority value. This is useful when using custom priority
	// values as categories.
	KeepOriginal,

	// Increment the original priority by a small amount (0.01). This can be useful when copying a patch
	// around multiple times, as it allows the new patches to be roughly in the same place in the 
	// priority hierarchy while still being higher priority than the copied patch.
	SmallIncrement,
};

/**
 * Base class for landscape patches: components that can be attached to meshes and moved around to make
 * the meshes affect the landscape around themselves.
 */
//~ TODO: Although this doesn't generate geometry, we are likely to change this to inherit from UPrimitiveComponent
//~ so that we can use render proxies for passing along data to the render thread or perhaps for visualization.
UCLASS(MinimalAPI, Blueprintable, BlueprintType, Abstract, HideCategories = (Tags, Activation, Cooking, Physics, Rendering, Navigation, LOD))
class ULandscapePatchComponent : public USceneComponent
	, public ILandscapeEditLayerRenderer
{
	GENERATED_BODY()

public:
	LANDSCAPEPATCH_API ULandscapePatchComponent(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

#if WITH_EDITOR
	// ILandscapeEditLayerRenderer
	// Subclasses are expected to implement GetEditLayerRendererDebugName, GetRendererStateInfo, GetRenderItems, and RenderLayer
#endif

	// These determine whether the patch is configured correctly to affect height/weightmaps,
	// ignoring whether it is currently enabled or not.
	virtual bool CanAffectHeightmap() const { return false; }
	virtual bool CanAffectWeightmap() const { return false; }
	virtual bool CanAffectWeightmapLayer(const FName& InLayerName) const { return false; }
	virtual bool CanAffectVisibilityLayer() const { return false; }

	bool CanAffectLandscape() const { return CanAffectHeightmap() || CanAffectWeightmap() || CanAffectVisibilityLayer(); }

	// These compose IsEnabled with the appropriate CanAffect functions
	bool AffectsHeightmap() const { return IsEnabled() && CanAffectHeightmap(); }
	bool AffectsWeightmap() const { return IsEnabled() && CanAffectWeightmap(); }
	bool AffectsWeightmapLayer(const FName& InLayerName) const { return IsEnabled() && CanAffectWeightmapLayer(InLayerName); }
	bool AffectsVisibilityLayer() const { return IsEnabled() && CanAffectVisibilityLayer(); }

	// Allows the patch to specify textures that need to be ready/compiled before applying the patch
	virtual void GetRenderDependencies(TSet<UObject*>& OutDependencies) const {}

	UFUNCTION(BlueprintCallable, Category = "LandscapePatch")
	LANDSCAPEPATCH_API void RequestLandscapeUpdate(bool bInUserTriggeredUpdate = false);

	/**
	 * Allows the patch to be disabled, so that it no longer affects the landscape. This can be useful
	 * when deleting the patch is undesirable, usually when the disabling is temporary.
	 */
	UFUNCTION(BlueprintCallable, Category = "LandscapePatch")
	LANDSCAPEPATCH_API void SetIsEnabled(bool bEnabledIn);

	/**
	 * @return false if the patch is marked as disabled and therefore can't affect the landscape.
	 */
	UFUNCTION(BlueprintCallable, Category = "LandscapePatch")
	virtual bool IsEnabled() const { return bIsEnabled; }

	UFUNCTION(BlueprintCallable, Category = "LandscapePatch")
	double GetPriority() const { return Priority; }

	UFUNCTION(BlueprintCallable, Category = "LandscapePatch")
	LANDSCAPEPATCH_API void SetPriority(double PriorityIn);

	UFUNCTION(BlueprintCallable, Category = "LandscapePatch")
	FGuid GetEditLayerGuid() const { return EditLayerGuid; }

	UFUNCTION(BlueprintCallable, Category = "LandscapePatch")
	LANDSCAPEPATCH_API void SetEditLayerGuid(const FGuid& GuidIn);

		
	/**
	 * Tries to assign the patch to the landscape 
	 * @param InLandscape Landscape actor on which the patch component should be applied. Passing an null actor will un-assign the patch component from any landscape
	 * @param InEditLayerName Name of the edit layer on which the patch component should be assigned. The layer name has to be present on the landscape and it has to be a Landscape Patch Edit Layer for this to succeed
	 * @return true if the patch was successfully assigned (or un-assigned) to (from) the landscape's edit layer
	 */
	UFUNCTION(BlueprintCallable, Category = "LandscapePatch")
	LANDSCAPEPATCH_API bool AssignToLandscape(ALandscape* InLandscape, const FName& InEditLayerName);

	/**
	 * Returns the result of GetTypeHash(GetFullName()) for this component. Useful for quick deterministic
	 * (though not lexical) ordering of same-priority patches. 
	 */
	uint32 GetFullNameHash() const;

	UFUNCTION(BlueprintCallable, Category = "LandscapePatch")
	LANDSCAPEPATCH_API FTransform GetLandscapeHeightmapCoordsToWorld() const;

	UFUNCTION(BlueprintCallable, Category = "LandscapePatch")
	ALandscape* GetLandscape() const { return Landscape.Get(); }

	UFUNCTION(BlueprintCallable, Category = "LandscapePatch")
	LANDSCAPEPATCH_API virtual void SetLandscape(ALandscape* NewLandscape);

	UE_DEPRECATED(all, "Patch manager use is deprecated. Patches should point to an edit layer using a Guid, and order themselves using priority.")
	UFUNCTION(BlueprintCallable, Category = "LandscapePatch", meta = (DeprecatedFunction, DeprecationMessage = "Patch manager use is deprecated. Patches should point to an edit layer using a Guid, and order themselves using priority."))
	LANDSCAPEPATCH_API virtual void SetPatchManager(ADEPRECATED_LandscapePatchManager* NewPatchManager);

	UE_DEPRECATED(all, "Patch manager use is deprecated. Patches should point to an edit layer using a Guid, and order themselves using priority.")
	UFUNCTION(BlueprintCallable, Category = "LandscapePatch", meta = (DeprecatedFunction, DeprecationMessage = "Patch manager use is deprecated. Patches should point to an edit layer using a Guid, and order themselves using priority."))
	LANDSCAPEPATCH_API virtual ADEPRECATED_LandscapePatchManager* GetPatchManager() const;

	// Verifies that we're not an archetype or trash or in an invalid world
	bool IsPatchInWorld() const;

	// For now we keep the patches largely editor-only, since we don't yet support runtime landscape editing.
	// The above functions are also editor-only (and don't work at runtime), but can't be in WITH_EDITOR blocks
	// so that they can be called from non-editor-only classes in editor contexts.
#if WITH_EDITOR

	// Cleanup method that can be used to fix the patch's binding to a landscape. An arbitrary landscape
	//  will be bound to if no guid, or no landscape is set on the patch.
	void FixBindings();

	// Allows the patch to update its UI and cached pointer
	void NotifyOfBoundLayerDeletion(ULandscapePatchEditLayer* LayerIn);

	// USceneComponent
	LANDSCAPEPATCH_API virtual void OnUpdateTransform(EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport) override;

	// UActorComponent
	LANDSCAPEPATCH_API virtual void CheckForErrors() override;
	LANDSCAPEPATCH_API virtual void OnComponentCreated() override;
	LANDSCAPEPATCH_API virtual void OnComponentDestroyed(bool bDestroyingHierarchy) override;
	LANDSCAPEPATCH_API virtual void OnRegister() override;
	LANDSCAPEPATCH_API virtual void GetActorDescProperties(FPropertyPairsMap& PropertyPairsMap) const override;
	LANDSCAPEPATCH_API virtual TStructOnScope<FActorComponentInstanceData> GetComponentInstanceData() const override;

	// UObject
	LANDSCAPEPATCH_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual bool IsPostLoadThreadSafe() const override { return true; }
	LANDSCAPEPATCH_API virtual void PostLoad() override;
	LANDSCAPEPATCH_API virtual void PostEditUndo() override;
#endif
	virtual bool IsEditorOnly() const override { return true; }
	virtual bool NeedsLoadForClient() const override { return false; }
	virtual bool NeedsLoadForServer() const override { return false; }
protected:

#if WITH_EDITOR
	ULandscapePatchEditLayer* GetBoundEditLayer() const { return EditLayer.Get(); }
#endif

	// Guid of the edit layer to which the patch is bound
	UPROPERTY()
	FGuid EditLayerGuid;

	/** How to initialize the patch priority when a patch is first created. */
	UPROPERTY(EditAnywhere, Category = Settings, AdvancedDisplay)
	ELandscapePatchPriorityInitialization PriorityInitialization = ELandscapePatchPriorityInitialization::AcquireHighest;

	// Value that determines the patch ordering relative to other patches. 
	UPROPERTY(EditAnywhere, Category = Settings)
	double Priority = ULandscapePatchEditLayer::PATCH_PRIORITY_BASE;

	UPROPERTY(EditAnywhere, Category = Settings)
	TSoftObjectPtr<ALandscape> Landscape = nullptr;

	/**
	 * When false, patch does not affect the landscape. Useful for temporarily disabling the patch.
	 */
	UPROPERTY(EditAnywhere, Category = Settings)
	bool bIsEnabled = true;

	// Determines whether the height patch was made by copying a different height patch.
	bool bWasCopy = false;

#if WITH_EDITORONLY_DATA
private:
	// Starts as false and gets set to true in construction, so gets used to set bWasCopy
	// by checking the indicator value at the start of construction.
	UPROPERTY()
	bool bPropertiesCopiedIndicator = false;

	/**
	 * Deprecated: Patch manager that can hold the patch as part of its legacy patch list.
	 * 
	 * Instead, patches should now point to an edit layer via a guid and order themselves using their
	 * priority value.
	 */
	UPROPERTY()
	TSoftObjectPtr<ADEPRECATED_LandscapePatchManager> PatchManager_DEPRECATED = nullptr;

	//~ Name of edit layer displayed to user
	/**
	 * Name of the edit layer to which the patch is bound. Options are determined by the set Landscape pointer.
	 */ 
	UPROPERTY(Transient, EditAnywhere, Category = Settings, Meta = (DisplayName="Edit Layer", GetOptions = "GetLayerOptions"))
	FString DetailPanelLayerName = TEXT("-Null-");

	UPROPERTY(Transient, VisibleAnywhere, Category = Settings, AdvancedDisplay, Meta = (DisplayName = "Layer Guid"))
	FString DetailPanelLayerGuid = EditLayerGuid.ToString();
#endif

	//~ Gives a drop down for edit layer names depending on the set landscape. Can't be editor only to be used in GetOptions
	UFUNCTION()
	const TArray<FString> GetLayerOptions();

#if WITH_EDITOR
	// Transient pointer to the edit layer that we are bound to using EditLayerGuid. Will
	// be null if that layer is not of an appropriate type.
	TWeakObjectPtr<ULandscapePatchEditLayer> EditLayer;

	bool BindToEditLayer(const FGuid& Guid);
	void ResetEditLayer();
	bool BindToLandscape(ALandscape* LandscapeIn);
	bool BindToAnyLandscape();
	void UpdateEditLayerFromDetailPanelLayerName();

	// Tells us whether the patch is attached to an actor that is being dragged around as an editor preview
	bool IsPatchPreview();

	// Used to avoid spamming warning messages
	bool bGaveCouldNotBindToEditLayerWarning = false;
	bool bGaveMissingEditLayerGuidWarning = false;
	bool bGaveMismatchedLandscapeWarning = false;
	void ResetWarnings();
	bool bInstanceDataApplied = false;
	bool bDeferUpdateRequestUntilInstanceData = false;
	
	void ApplyComponentInstanceData(struct FLandscapePatchComponentInstanceData* ComponentInstanceData, ECacheApplyPhase CacheApplyPhase);
#endif

	friend struct FLandscapePatchComponentInstanceData;
};

/** Used to store some extra data during RerunConstructionScripts. */
USTRUCT()
struct FLandscapePatchComponentInstanceData : public FSceneComponentInstanceData
{
	GENERATED_BODY()

	FLandscapePatchComponentInstanceData() = default;
	FLandscapePatchComponentInstanceData(const ULandscapePatchComponent* SourceComponent);
	
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// These are necessary to prevent deprecation warnings from UE_DEPRECATED(5.7, ...) on some compilers because of auto-generated code. Can be removed after the deprecated members are removed :
	FLandscapePatchComponentInstanceData(const FLandscapePatchComponentInstanceData&) = default;
	FLandscapePatchComponentInstanceData(FLandscapePatchComponentInstanceData&&) = default;
	FLandscapePatchComponentInstanceData& operator=(const FLandscapePatchComponentInstanceData&) = default;
	FLandscapePatchComponentInstanceData& operator=(FLandscapePatchComponentInstanceData&&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	virtual ~FLandscapePatchComponentInstanceData() = default;

	virtual bool ContainsData() const override
	{
		return true;
	}

	virtual void ApplyToComponent(UActorComponent* Component, const ECacheApplyPhase CacheApplyPhase) override
	{
#if WITH_EDITOR
		Super::ApplyToComponent(Component, CacheApplyPhase);
		CastChecked<ULandscapePatchComponent>(Component)->ApplyComponentInstanceData(this, CacheApplyPhase);
#endif
	}

#if WITH_EDITORONLY_DATA
	// The UPROPERTY tags inside a FSceneComponentInstanceData might not be necessary, but might 
	// potentially be used in some multiuser code paths.

	UPROPERTY()
	TSoftObjectPtr<ADEPRECATED_LandscapePatchManager> PatchManager_DEPRECATED = nullptr;
	UPROPERTY()
	FGuid EditLayerGuid;
	// Priority needs to be carried over becuse our tweaks to it in OnComponentCreated cause it to not be
	//  captured automatically, as it is detected as "UCS altered".
	UPROPERTY()
	double Priority = 0;

	// Used so that we don't spam warning messages while rerunning construction scripts on a patch
	// that triggers one of the warnings.
	UPROPERTY()
	bool bGaveCouldNotBindToEditLayerWarning = false;
	UPROPERTY()
	bool bGaveMismatchedLandscapeWarning = false;
	UPROPERTY()
	bool bGaveMissingEditLayerGuidWarning = false;

	UE_DEPRECATED(5.7, "Not needed since the deprecation of PatchManager")
	UPROPERTY()
	bool bGaveNotInPatchManagerWarning = false;
	UE_DEPRECATED(5.7, "Not needed since the deprecation of PatchManager")
	UPROPERTY()
	bool bGaveMissingLandscapeWarning = false;
#endif
};
