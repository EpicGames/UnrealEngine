// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LandscapeEditLayer.h"
#include "Misc/Guid.h"

#include "LandscapePatchEditLayer.generated.h"

class ULandscapePatchComponent;

/**
 * Special edit layer used only for landscape patches. Patches point to the layer via a guid
 *  and determine their ordering by their Priority values.
 */
UCLASS(MinimalAPI, meta = (ShortToolTip = "Special edit layer for landscape patches"))
class ULandscapePatchEditLayer : public ULandscapeEditLayerProcedural
{
	GENERATED_BODY()

public:

#if WITH_EDITOR
	/**
	 * Must be called by patches on this layer to let the layer know that the patch is pointing
	 *  to it via its layer guid.
	 */
	virtual void RegisterPatchForEditLayer(ULandscapePatchComponent* Patch);

	/**
	 * Should be called to notify the edit layer when a patch is destroyed or changes its edit layer guid
	 *  away from this layer.
	 */
	virtual void NotifyOfPatchRemoval(ULandscapePatchComponent* Patch);

	/**
	 * Should be called to notify the edit layer when a patch priority changes.
	 */
	virtual void NotifyOfPriorityChange(ULandscapePatchComponent* Patch);

	/**
	 * Returns the highest known patch priority, or PATCH_PRIORITY_BASE if none are higher.
	 */
	virtual double GetHighestPatchPriority();

	void RequestLandscapeUpdate(bool bInUserTriggered = false);
#endif // WITH_EDITOR

	inline static const double PATCH_PRIORITY_BASE = 1000;

	// ULandscapeEditLayerBase
	virtual bool SupportsTargetType(ELandscapeToolTargetType InType) const override;
	virtual bool NeedsPersistentTextures() const override { return false; }
	virtual bool SupportsMultiple() const override { return true; }
	virtual FString GetDefaultName() const override { return TEXT("Patches"); }
	virtual void GetRenderDependencies(TSet<UObject*>& OutDependencies) const override;
	virtual void OnLayerRemoved() override;

#if WITH_EDITOR
	using FEditLayerRendererState = UE::Landscape::EditLayers::FEditLayerRendererState;

	// IEditLayerRendererProvider
	virtual TArray<FEditLayerRendererState> GetEditLayerRendererStates(const UE::Landscape::EditLayers::FMergeContext* InMergeContext) override;

	// UObject
	virtual void PostEditUndo() override;
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA

private:
	// This is the transient list of patches that are bound to this edit layer, usually sorted by priority.
	//  TSoftObjectPtr is used because it is robust across blueprint actor construction script reruns.
	UPROPERTY(Transient)
	TArray<TSoftObjectPtr<ULandscapePatchComponent>> RegisteredPatches;

	// A helper structure for quick containment queries and updates. This should always be kept consistent with
	//  RegisteredPatches.
	UPROPERTY(Transient)
	TMap<TSoftObjectPtr<ULandscapePatchComponent>, int32> PatchToIndex;

	// When true, the entire patch list needs filtering and sorting. This is not meant to happen but it
	//  is used as a safety in case some lack of notifications puts us in a situation where our patches
	//  are not all valid and sorted.
	bool bPatchListDirty = false;
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
	void UpdatePatchList();
	void UpdatePatchListIfDirty();
	void UpdateHighestKnownPriority();

	bool ShouldPatchBeIncludedInList(const ULandscapePatchComponent* Patch) const;
	int32 GetInsertionIndex(ULandscapePatchComponent* Patch, bool& bFoundInvalidPatchOut) const;

	// A tracker of the highest priority we've seen. When the patch list is not dirty, it will be the priority
	//  of the last element, but this variable allows us to maintain the value past dirtying.
	double HighestKnownPriority = PATCH_PRIORITY_BASE;
#endif // WITH_EDITOR
};
