// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"

LANDSCAPEPATCH_API DECLARE_LOG_CATEGORY_EXTERN(LogLandscapePatch, Log, All);
