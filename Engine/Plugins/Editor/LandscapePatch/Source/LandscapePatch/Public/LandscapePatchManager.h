// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "LandscapeBlueprintBrushBase.h"
#include "LandscapeEditTypes.h"
#include "LandscapePatchEditLayer.h" // PATCH_PRIORITY_BASE

#include "LandscapePatchManager.generated.h"

class ALandscape;
class ULandscapePatchComponent;

/**
 * Actor used in legacy landscape patch handling where a manager keeps a serialized list
 * of patches that determines their priority. This approach is deprecated- patches now
 * point to a special landscape patch edit layer via a guid, and determine their ordering
 * relative to each other using a priority value.
 */
UCLASS(Deprecated, meta = (DeprecationMessage = "Landscape patches are now fully handled by ULandscapePatchEditLayer and migration is automatic so effectively, this actor is only there for backwards-compatibility/serialization purposes"))
class ADEPRECATED_LandscapePatchManager : public ALandscapeBlueprintBrushBase
{
	GENERATED_BODY()

public:
	ADEPRECATED_LandscapePatchManager(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

#if WITH_EDITOR
	// IEditLayerRendererProvider
	// Deprecated
	virtual TArray<UE::Landscape::EditLayers::FEditLayerRendererState> GetEditLayerRendererStates(const UE::Landscape::EditLayers::FMergeContext* InMergeContext) override
	{ 
		return {}; 
	}

	// ILandscapeEditLayerRenderer
	// Deprecated
	virtual void GetRendererStateInfo(const UE::Landscape::EditLayers::FMergeContext* InMergeContext,
		UE::Landscape::EditLayers::FEditLayerTargetTypeState& OutSupportedTargetTypeState,
		UE::Landscape::EditLayers::FEditLayerTargetTypeState& OutEnabledTargetTypeState,
		TArray<UE::Landscape::EditLayers::FTargetLayerGroup>& OutTargetLayerGroups) const override 
	{}

	// Deprecated
	virtual TArray<UE::Landscape::EditLayers::FEditLayerRenderItem> GetRenderItems(const UE::Landscape::EditLayers::FMergeContext* InMergeContext) const override
	{ 
		return {}; 
	}

	// Deprecated
	virtual bool RenderLayer(UE::Landscape::EditLayers::FRenderParams& RenderParams, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder) override 
	{ 
		return false; 
	};

	// In 5.6 users should not be able to add new Patch Managers anywhere in the editor
	virtual bool SupportsBlueprintBrushTool() const { return false; }
#endif // WITH_EDITOR

	// Adds the brush to the given landscape, removing it from any previous one. This differs from SetOwningLandscape
	// in that SetOwningLandscape is called by the landscape itself from AddBrushToLayer to update the manager.
	UE_DEPRECATED(5.7, "Patch manager is deprecated and replaced by Landscape Patch Edit Layer.")
	UFUNCTION(BlueprintCallable, Category = LandscapeManager, meta = (DeprecatedFunction, DeprecationMessage = "Patch manager is deprecated and replaced by Landscape Patch Edit Layer."))
	virtual void SetTargetLandscape(ALandscape* InOwningLandscape);

	// Deprecated
	UE_DEPRECATED(5.7, "LandscapePatchManager is deprecated and replaced by Landscape Patch Edit Layer.")
	virtual FTransform GetHeightmapCoordsToWorld() 
	{ 
		return FTransform(); 
	}

	UE_DEPRECATED(5.7, "Patch manager is deprecated and replaced by Landscape Patch Edit Layer.")
	UFUNCTION(BlueprintCallable, Category = LandscapePatch, meta = (DeprecatedFunction, DeprecationMessage = "Patch manager is deprecated and replaced by Landscape Patch Edit Layer."))
	bool ContainsPatch(ULandscapePatchComponent* Patch) const
	{
		return false;
	}

	UE_DEPRECATED(5.7, "Patch manager is deprecated and replaces by Landscape Patch Edit Layer.")
	UFUNCTION(BlueprintCallable, Category = LandscapePatch, meta = (DeprecatedFunction, DeprecationMessage = "Patch manager is deprecated and replaced by Landscape Patch Edit Layer."))
	void AddPatch(ULandscapePatchComponent* Patch)
	{}

	UE_DEPRECATED(5.7, "Patch manager is deprecated and replaces by Landscape Patch Edit Layer.")
	UFUNCTION(BlueprintCallable, Category = LandscapePatch, meta = (DeprecatedFunction, DeprecationMessage = "Patch manager is deprecated and replaced by Landscape Patch Edit Layer."))
	bool RemovePatch(ULandscapePatchComponent* Patch)
	{
		return false;
	}

	UE_DEPRECATED(5.7, "Patch manager is deprecated and replaces by Landscape Patch Edit Layer.")
	UFUNCTION(BlueprintCallable, Category = LandscapePatch, meta = (DeprecatedFunction, DeprecationMessage = "Patch manager is deprecated and replaced by Landscape Patch Edit Layer."))
	int32 GetIndexOfPatch(const ULandscapePatchComponent* Patch) const
	{
		return INDEX_NONE;
	}

	UE_DEPRECATED(5.7, "Patch manager is deprecated and replaces by Landscape Patch Edit Layer.")
	UFUNCTION(BlueprintCallable, Category = LandscapePatch, meta = (DeprecatedFunction, DeprecationMessage = "Patch manager use is deprecated and replaces by Landscape Patch Edit Layer."))
	void MovePatchToIndex(ULandscapePatchComponent* Patch, int32 Index)
	{}

#if WITH_EDITOR

	/**
	 * Move any patches from legacy patch list to being bound directly to an edit layer,
	 * and delete the patch manager. This will cause a popup to the user if there is still
	 * a dangling reference to the manager (there shouldn't be).
	 */
	UFUNCTION(CallInEditor, Category = LandscapeManager, meta = (DisplayName = "MigrateToPrioritySystem"))
	void MigrateToPrioritySystemAndDelete();

	// ALandscapeBlueprintBrushBase
	virtual bool CanAffectWeightmapLayer(const FName& InLayerName) const override
	{
		return false;
	}

	virtual bool AffectsHeightmap() const override
	{
		return false;
	}

	virtual bool AffectsWeightmap() const override
	{
		return false;
	}

	virtual bool AffectsWeightmapLayer(const FName& InLayerName) const override
	{
		return false;
	}

	virtual bool AffectsVisibilityLayer() const override
	{
		return false;
	}

	virtual void GetRenderDependencies(TSet<UObject*>& OutDependencies) override
	{}

	// AActor
	virtual void PostRegisterAllComponents() override;

#endif
	virtual bool IsEditorOnly() const override 
	{ 
		return true; 
	}

	virtual bool NeedsLoadForClient() const override 
	{ 
		return false; 
	}

	virtual bool NeedsLoadForServer() const override 
	{ 
		return false; 
	}

	// This is intentionally lower than PATCH_PRIORITY_BASE so that patches converted from a
	// patch manager list are applied before other edit layer patches.
	inline static const double LEGACY_PATCH_PRIORITY_BASE = ULandscapePatchEditLayer::PATCH_PRIORITY_BASE - 10;

private:

	void MigrateToPrioritySystemAndDeleteInternal(bool bAllowUI);

	UPROPERTY()
	TArray<TSoftObjectPtr<ULandscapePatchComponent>> PatchComponents;

#if WITH_EDITORONLY_DATA
	// This patch manager has been migrated out of and should no longer be accessible.
	bool bDead = false;
#endif
};
