// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h" // FTransform

#include "LandscapePatchUtil.generated.h"

class FRHICommandListImmediate;
class FTextureResource;


// ----------------------------------------------------------------------------------

// Values needed to convert a patch stored in some source encoding into the native (two byte int) encoding and back
USTRUCT()
struct FLandscapeHeightPatchConvertToNativeParams
{
	GENERATED_BODY()

	UPROPERTY()
	float ZeroInEncoding = 0.0f;

	UPROPERTY()
	float HeightScale = 1.0f;

	UPROPERTY()
	float HeightOffset = 0.0f;
};


// ----------------------------------------------------------------------------------

namespace UE:: Landscape::PatchUtil 
{
	void CopyTextureOnRenderThread(FRHICommandListImmediate& RHICmdList, const FTextureResource& Source, FTextureResource& Destination);

	/**
	 * Given a landscape transform, gives a transform from heightmap coordinates (where the Z value is the
	 * two byte integer value stored as the height) to world coordinates.
	 */
	FTransform GetHeightmapToWorld(const FTransform& InLandscapeTransform);

}//end UE::Landscape::PatchUtil
