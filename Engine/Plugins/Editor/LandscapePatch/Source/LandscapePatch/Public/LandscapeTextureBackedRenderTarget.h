// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "Engine/TextureRenderTarget2D.h" // ETextureRenderTargetFormat
#include "LandscapePatchUtil.h" // FLandscapeHeightPatchConvertToNativeParams

#include "LandscapeTextureBackedRenderTarget.generated.h"

class UTexture2D;
class UTextureRenderTarget2D;

/**
 * A combination of a render target and UTexture2D that allows render target to be saved across save/load/etc
 * by copying the data back and forth from the internal texture.
 * 
 * After an Initialize() call, the internal texture will always be present, and the render target will be
 * present depending on how SetUseInternalTextureOnly is called (by default, present). 
 */
UCLASS(Abstract)
class ULandscapeTextureBackedRenderTargetBase : public UObject
{
	GENERATED_BODY()

public:

	virtual UTextureRenderTarget2D* GetRenderTarget() { return RenderTarget; }
	virtual UTexture2D* GetInternalTexture() { return InternalTexture; }

	/**
	 * 
	 */
	virtual void SetUseInternalTextureOnly(bool bUseInternalTextureOnlyIn, bool bCopyExisting = true);

	virtual void SetResolution(int32 SizeXIn, int32 SizeYIn);

	virtual void Initialize();

	virtual void CopyToInternalTexture() PURE_VIRTUAL(ULandscapeTextureBackedRenderTargetBase::CopyToInternalTexture);
	virtual void CopyBackFromInternalTexture() PURE_VIRTUAL(ULandscapeTextureBackedRenderTargetBase::CopyBackFromInternalTexture);

#if WITH_EDITOR
	// UObject
	virtual void PreSave(FObjectPreSaveContext SaveContext) override;
	virtual void PostLoad() override;
	virtual void PreDuplicate(FObjectDuplicationParameters& DupParams) override;
	virtual void ExportCustomProperties(FOutputDevice& Out, uint32 Indent) override;
	virtual void PostEditImport() override;
	virtual void PostEditUndo() override;
#endif // WITH_EDITOR

	// This is a very bespoke function to work around UE-225445, see comment in cpp file
	UE_INTERNAL void SetTemporarilyNonTransactional(bool bInIsTemporarilyNonTransactional);

protected:
	// Allows us to disable copying back and forth between render target and texture when this is not possible
	// (for instance, when rendering is turned off, so render target resources are not allocated).
	virtual bool IsCopyingBackAndForthAllowed();

	virtual ETextureSourceFormat GetInternalTextureFormat() PURE_VIRTUAL(ULandscapeTextureBackedRenderTargetBase::GetInternalTextureFormat, return ETextureSourceFormat::TSF_G8;);
	virtual ETextureRenderTargetFormat GetRenderTargetFormat() PURE_VIRTUAL(ULandscapeTextureBackedRenderTargetBase::GetRenderTargetFormat, return ETextureRenderTargetFormat::RTF_R8;);

	void ReinitializeRenderTarget(bool bClear);

#if WITH_EDITOR
	UTexture2D* CreateInternalTexture();
#endif // WITH_EDITOR

public:
	UPROPERTY(VisibleAnywhere, Category = DebugData)
	TObjectPtr<UTexture2D> InternalTexture = nullptr;

	UPROPERTY(VisibleAnywhere, Category = DebugData, Transient, DuplicateTransient)
	TObjectPtr<UTextureRenderTarget2D> RenderTarget = nullptr;

	UPROPERTY(VisibleAnywhere, Category = DebugData)
	int32 SizeX = 32;

	UPROPERTY(VisibleAnywhere, Category = DebugData)
	int32 SizeY = 32;

	UPROPERTY()
	bool bUseInternalTextureOnly = false;

protected:
	UPROPERTY()
	// Allows to indicate that the object has temporarily lost its RF_Transactional flag and should not add it back, neither to itself or to 
	//  any of its sub-objects at this time (see SetTemporarilyNonTransactional)
	bool bIsTemporarilyNonTransactional = false;
};

UCLASS()
class ULandscapeWeightTextureBackedRenderTarget : public ULandscapeTextureBackedRenderTargetBase
{
	GENERATED_BODY()

public:

	virtual void SetUseAlphaChannel(bool bUseAlphaChannelIn);

	virtual void Initialize() override;
	virtual void CopyToInternalTexture() override;
	virtual void CopyBackFromInternalTexture() override;
protected:
	virtual ETextureSourceFormat GetInternalTextureFormat() override;
	virtual ETextureRenderTargetFormat GetRenderTargetFormat() override;

	UPROPERTY()
	bool bUseAlphaChannel = true;
};

UCLASS()
class ULandscapeHeightTextureBackedRenderTarget : public ULandscapeTextureBackedRenderTargetBase
{
	GENERATED_BODY()

public:

	virtual void SetFormat(ETextureRenderTargetFormat FormatToUse);

	virtual void Initialize() override;
	virtual void CopyToInternalTexture() override;
	virtual void CopyBackFromInternalTexture() override;

	UPROPERTY()
	FLandscapeHeightPatchConvertToNativeParams ConversionParams;

protected:
	virtual ETextureSourceFormat GetInternalTextureFormat() override { return ETextureSourceFormat::TSF_BGRA8; }
	virtual ETextureRenderTargetFormat GetRenderTargetFormat() override { return RenderTargetFormat; }

	UPROPERTY()
	TEnumAsByte<ETextureRenderTargetFormat> RenderTargetFormat = ETextureRenderTargetFormat::RTF_R32f;
};
