// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "LandscapePatchComponent.h"
#include "LandscapeTextureBackedRenderTarget.h"
#include "RHIAccess.h"

#include "LandscapeTexturePatch.generated.h"

class FTextureResource;
class ULandscapeTexturePatch;
class ULandscapeHeightTextureBackedRenderTarget;
class UTexture;
class UTexture2D;

namespace UE::Landscape
{
	class FApplyLandscapeTextureHeightPatchPSParameters;
	class FApplyLandscapeTextureWeightPatchPSParameters;
	class FRDGBuilderRecorder;
} // namespace UE::Landscape

/**
 * Determines where the patch gets its information, which affects its memory usage in editor (not in runtime,
 * since patches are baked directly into landscape and removed for runtime).
 */
UENUM(BlueprintType)
enum class ELandscapeTexturePatchSourceMode : uint8
{
	/**
	 * The patch is considered not to have any data stored for this element. Setting source mode to this is
	 * a way to discard any internally stored data.
	 */
	None,

	/**
	 * The data will be read from an internally-stored UTexture2D. In this mode, the patch can't be written-to via
	 * blueprints, but it avoids storing the extra render target needed for TextureBackedRenderTarget.
	 */
	InternalTexture,

	/**
	* The patch data will be read from an internally-stored render target, which can be written to via Blueprints
	* and which gets serialized to an internally stored UTexture2D when needed. Uses double the memory of InternalTexture.
	*/
	TextureBackedRenderTarget,

	/**
	 * The data will be read from a UTexture asset (which can be a render target). Allows multiple patches
	 * to share the same texture.
	 */
	 TextureAsset
};

/**
 * Determines the alpha mask applied to a patch texture. Alpha is applied in addition to the patch falloff settings
 */
UENUM(BlueprintType)
enum class ELandscapeTexturePatchAlphaSourceMode : uint8
{
	/**
	 * The patch texture is not masked by any alpha value
	 */
	None,

	/**
	 * The alpha data will be read from an internally-stored UTexture2D. In this mode, the patch can't be written-to via
	 * blueprints, but it avoids storing the extra render target needed for TextureBackedRenderTarget.
	 */
	InternalTexture,

	/**
	* The patch alpha data will be read from an internally-stored render target, which can be written to via Blueprints
	* and which gets serialized to an internally stored UTexture2D when needed. Uses double the memory of InternalTexture.
	*/
	TextureBackedRenderTarget,

	/**
	 * The alpha data will be read from a UTexture asset (which can be a render target). Allows multiple patches
	 * to share the same alpha texture.
	 */
	TextureAsset,

	/**
	 * A color channel of the original source texture will be used as the alpha mask. By default, the alpha channel of the
	 * texture is used. The source texture must have data in the defined channel to affect the patch
	 */
	SourceTextureChannel
};

/**
 * Color channel of the source texture
 */
UENUM(BlueprintType)
enum class ELandscapeTexturePatchTextureChannel : uint8
{
	None UMETA(Hidden),
	// Texture's red channel
	Red,
	// Texture's green channel
	Green,
	// Texture's blue channel
	Blue, 
	// Texture's alpha channel
	Alpha, 
	// Texture's luminance 
	Luminance
};

// Determines how the patch is combined with the previous state of the landscape.
UENUM(BlueprintType)
enum class ELandscapeTexturePatchBlendMode : uint8
{
	// Let the patch specify the actual target height, and blend that with the existing
	// height using falloff/alpha. E.g. with no falloff and alpha 1, the landscape will
	// be set directly to the height sampled from patch. With alpha 0.5, landscape height 
	// will be averaged evenly with patch height.
	AlphaBlend,

	// Interpreting the landscape mid value as 0, use the texture patch as an offset to
	// apply to the landscape. Falloff/alpha will just affect the degree to which the offset
	// is applied (e.g. alpha of 0.5 will apply just half the offset).
	Additive,

	// Like Alpha Blend mode, but limited to only lowering the existing landscape values
	Min,

	// Like Alpha Blend mode, but limited to only raising the existing landscape values
	Max
};

// Determines falloff method for the patch's influence.
UENUM(BlueprintType)
enum class ELandscapeTexturePatchFalloffMode : uint8
{
	// Affect landscape in a circle inscribed in the patch, and fall off across
	// a margin extending into that circle.
	Circle,

	// Affect entire rectangle of patch (except for circular corners), and fall off
	// across a margin extending inward from the boundary.
	RoundedRectangle,
};

UENUM(BlueprintType)
enum class ELandscapeTextureHeightPatchEncoding : uint8
{
	// Values in texture should be interpreted as being floats in the range [0,1]. User specifies what
	// value corresponds to height 0 (i.e. height when landscape is "cleared"), and the size of the 
	// range in world units.
	ZeroToOne,

	// Values in texture are direct world-space heights.
	WorldUnits,

	// Values in texture are stored the same way they are in landscape actors: as 16 bit integers packed 
	// into two bytes, mapping to [-256, 256 - 1/128] before applying landscape scale.
	NativePackedHeight

	//~ Note that currently ZeroToOne and WorldUnits actually work the same way- we subtract the center point (0 for WorldUnits),
	//~ then scale in some way (1.0 for WorldUnits). However, having separate options here allows us to initialize defaults
	//~ appropriately when setting the encoding mode via ResetSourceEncodingMode.
};

UENUM(BlueprintType)
enum class ELandscapeTextureHeightPatchZeroHeightMeaning : uint8
{
	// Zero height corresponds to the patch vertical position relative to the landscape. This moves
	// the results up and down as the patch moves up and down.
	PatchZ,

	// Zero height corresponds to Z = 0 in the local space of the landscape, regardless of the patch vertical
	// position. For instance, if landscape transform has z=-100 in world, then writing height 0 will correspond
	// to z=-100 in world coordinates, regardless of patch Z. 
	LandscapeZ,

	// Zero height corresponds to the height of the world origin relative to landscape. In other words, writing
	// height 0 will correspond to world z = 0 regardless of patch Z or landscape transform (as long as landscape
	// transform still has Z up in world coordinates).
	WorldZero
};

USTRUCT(BlueprintType)
struct FLandscapeTexturePatchTextureTransform
{
	GENERATED_BODY()
public:
	/** When enabled, texture UVs are scaled using world-space units (1000 = tile texture every 10 meters). If disabled, scaling is applied in local UV space. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Texture Transform", meta = (DisplayName = "Use World Unit Tiling"))
	bool bUseWorldSpaceScale = false;

	/** By default, texture UVs are centered at the patch's min bounds. Moving the patch does not change texture sampling. 
	*  When enabled, UVs are anchored in world space at the world origin. Moving the patch's world position changes where the texture is sampled.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Texture Transform")
	bool bUseWorldPositionSampling = false;

	/** Scale applied to the texture's UV. By default, scaling is applied in local UV space. Relative scale 2.0 = texture is twice as large as original size. 
	*  When World Space Scale is enabled, scale 1000 = tile texture every 10 meters. 
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Texture Transform")
	FVector2D Scale = FVector2D(1, 1);
	
	/** Rotation in degrees applied to the texture's UV. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Texture Transform", meta = (UIMin = "0", UIMax = "360", ClampMin = "0", ClampMax = "360"))
	float Rotation = 0;

	/** Offset applied to the sampled texture. By default, offset is in UV space [0, 1]. When bUseWorldPositionSampling is enabled, offset is in world units. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Texture Transform")
	FVector2D Offset = FVector2D(0, 0);

	FMatrix44d GetTransformAsMatrix(const FTransform& PatchToWorldIn, const FVector2f& PatchWorldDimensionsIn) const;

	FORCEINLINE bool operator==(const FLandscapeTexturePatchTextureTransform InTransform) const
	{
		return bUseWorldSpaceScale == InTransform.bUseWorldSpaceScale &&
			bUseWorldPositionSampling == InTransform.bUseWorldPositionSampling &&
			Scale.X == InTransform.Scale.X && Scale.Y == InTransform.Scale.Y && 
			Offset.X == InTransform.Offset.X && Offset.Y == InTransform.Offset.Y &&
			Rotation == InTransform.Rotation;
	}
};

USTRUCT(BlueprintType)
struct FLandscapeTexturePatchAlphaSettings
{
	GENERATED_BODY()
public:
	/** When alpha source mode is TextureBackedRenderTarget returns the alpha render target. Will allocate new render target if not yet initialized */
	UTextureRenderTarget2D* GetAlphaRenderTarget(UObject* InOuter, bool bAlwaysMarkDirty = true);
	/** When alpha source mode is  InternalTexture/TextureBackedRenderTarget returns the alpha internal texture. Will allocate new internal data if not yet initialized */
	UTexture2D* GetAlphaInternalTexture(UObject* InOuter, bool bAlwaysMarkDirty = true);

	/** Transitions from the previous alpha source mode to the new mode. Marks the Outer owner dirty and clears internal data if needed  
	* @param InNewSourceMode the new alpha source mode
	* @param InOuter The alpha settings owner (must be direct texture patch or weight patch info)
	* @param bAlwaysMarkDirty When true, marks the outer package dirty when the source mode changes
	*/
	void SetAlphaSourceMode(ELandscapeTexturePatchAlphaSourceMode InNewSourceMode, UObject* InOuter, bool bAlwaysMarkDirty = true);
	ELandscapeTexturePatchAlphaSourceMode GetAlphaSourceMode() const
	{
		return AlphaSourceMode;
	};

	/** How the alpha texture is stored. */
	UPROPERTY(EditAnywhere, Category = "Texture Alpha Settings", meta = (DisplayName = "Source Mode"))
	ELandscapeTexturePatchAlphaSourceMode DetailPanelAlphaSourceMode = ELandscapeTexturePatchAlphaSourceMode::None;

	/** Alpha Texture to use when the source mode is set to texture asset. */
	UPROPERTY(EditAnywhere, Category = "Texture Alpha Settings", meta = (EditConditionHides,
		EditCondition = "AlphaSourceMode == ELandscapeTexturePatchAlphaSourceMode::TextureAsset", 
		DisallowedAssetDataTags = "VirtualTextureStreaming=True"))
	TObjectPtr<UTexture> AlphaTextureAsset;

	/** Texture channel utilized for the alpha mask. The texture alpha channel is used by default. */
	UPROPERTY(EditAnywhere, Category = "Texture Alpha Settings", meta = (EditConditionHides, EditCondition = "AlphaSourceMode != ELandscapeTexturePatchAlphaSourceMode::None"))
	ELandscapeTexturePatchTextureChannel AlphaTextureChannel = ELandscapeTexturePatchTextureChannel::Alpha;

	/** Not directly settable via detail panel- for alpha display/debugging purposes only. */
	UPROPERTY(VisibleAnywhere, Category = "Texture Alpha Settings", Instanced, AdvancedDisplay, meta = (DisplayName = "Debug Data", EditConditionHides, EditCondition = "AlphaSourceMode != ELandscapeTexturePatchAlphaSourceMode::None"))
	TObjectPtr<ULandscapeWeightTextureBackedRenderTarget> InternalAlphaData;

	/* Alpha texture's UV transform (Scale, Offset, Rotation) */
	UPROPERTY(EditAnywhere, Category = "Texture Alpha Settings", meta = (DisplayName = "Alpha Texture Transform"), meta = (EditConditionHides, EditCondition = "AlphaSourceMode != ELandscapeTexturePatchAlphaSourceMode::None"))
	FLandscapeTexturePatchTextureTransform AlphaTextureUVTransform;

private:
	/** When the source mode changes, there are transition steps that need run to clean up render targets / internal data.
	* In PostEditChangeProperty, the previous source mode is already lost and it becomes harder to track the necessary transition.
	* DetailPanelAlphaSourceMode exposes the property and allows us to process the source mode transition before updating AlphaSourceMode
	*/
	UPROPERTY(VisibleAnywhere, Category = "Texture Alpha Settings", meta = (EditConditionHides, EditCondition = "false"))
	ELandscapeTexturePatchAlphaSourceMode AlphaSourceMode = ELandscapeTexturePatchAlphaSourceMode::None;
};

//~ A struct in case we find that we need other encoding settings.
USTRUCT(BlueprintType)
struct FLandscapeTexturePatchEncodingSettings
{
	GENERATED_BODY()
public:
	/**
	 * The value in the patch data that corresponds to 0 height relative to the starting point
	 * specified by Zero Height Meaning.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	double ZeroInEncoding = 0;

	/**
	 * The scale that should be applied to the data stored in the patch relative to the zero in the encoding, in world coordinates.
	 * For instance if the encoding is [0,1], and 0.5 corresponds to 0, a WorldSpaceEncoding Scale of 100 means that the resulting
	 * values will lie in the range [-50, 50] in world space, which would be [-0.5, 0.5] in the landscape local heights if the Z
	 * scale is 100.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Settings)
	double WorldSpaceEncodingScale = 1;
};

//~ Ideally this would be a nested class, but it needs to be a UObject, which can't be nested.
/**
 * Helper class for ULandscapeTexturePatch that stores information for a given weight layer.
 * Should not be used outside this class.
 */
UCLASS(MinimalAPI, EditInlineNew, CollapseCategories)
class ULandscapeWeightPatchTextureInfo : public UObject
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	// UObject
	LANDSCAPEPATCH_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	LANDSCAPEPATCH_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
	LANDSCAPEPATCH_API virtual void PreDuplicate(FObjectDuplicationParameters& DupParams) override;
#endif // WITH_EDITOR

protected:
	/** Name of the target layer to affect. */
	UPROPERTY(EditAnywhere, Category = WeightPatch, meta = (DisplayName = "Target Layer Name"))
	FName WeightmapLayerName;

	/** Specifies if this patch affects the visibility layer. */
	UPROPERTY(EditAnywhere, Category = WeightPatch, meta = (DisplayName = "Affects Visibility Layer"))
	bool bEditVisibilityLayer = false;
	
	UPROPERTY(EditAnywhere, Category = WeightPatch, meta = (EditConditionHides, EditCondition = "false"))
	ELandscapeTexturePatchSourceMode SourceMode = ELandscapeTexturePatchSourceMode::None;

	/** How the weightmap of the patch is stored. */
	UPROPERTY(EditAnywhere, Category = WeightPatch, meta = (DisplayName = "Source Mode"))
	ELandscapeTexturePatchSourceMode DetailPanelSourceMode = ELandscapeTexturePatchSourceMode::None;

	/** Texture to use when source mode is set to texture asset. */
	UPROPERTY(EditAnywhere, Category = WeightPatch, meta = (EditConditionHides,
		EditCondition = "SourceMode == ELandscapeTexturePatchSourceMode::TextureAsset", 
		DisallowedAssetDataTags = "VirtualTextureStreaming=True"))
	TObjectPtr<UTexture> TextureAsset = nullptr;

	UPROPERTY(EditAnywhere, Category = WeightPatch, meta = (DisplayName = "Texture Transform"), meta = (EditConditionHides, EditCondition = "SourceMode != ELandscapeTexturePatchSourceMode::None"))
	FLandscapeTexturePatchTextureTransform WeightTextureUVTransform;

	/** Color channel of the texture used for weight data. By default the weight texture's red channel is used. */
	UPROPERTY(EditAnywhere, Category = HeightPatch, meta = (EditConditionHides, EditCondition = "SourceMode != ELandscapeTexturePatchSourceMode::None"))
	ELandscapeTexturePatchTextureChannel WeightPatchTextureChannel = ELandscapeTexturePatchTextureChannel::Red;

	/** Not directly settable via detail panel- for display/debugging purposes only. */
	UPROPERTY(VisibleAnywhere, Category = WeightPatch, Instanced, AdvancedDisplay, meta = (DisplayName = "Debug Data"))
	TObjectPtr<ULandscapeWeightTextureBackedRenderTarget> InternalData = nullptr;

	UE_DEPRECATED(5.8, "bUseAlphaChannel is deprecated, use the WeightPatchAlphaSettings.SetAlphaSourceMode() with ELandscapeTexturePatchAlphaSourceMode (use alpha mode None or SourceTextureChannel for legacy behavior).")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use WeightPatchAlphaSettings.SetAlphaSourceMode() with ELandscapeTexturePatchAlphaSourceMode (use alpha mode None or SourceTextureChannel for legacy behavior)."))
	bool bUseAlphaChannel_DEPRECATED = false;

	UPROPERTY(EditAnywhere, Category = WeightPatch, meta = (InlineEditConditionToggle))
	bool bOverrideWeightAlphaSettings = false;

	UPROPERTY(EditAnywhere, Category = WeightPatch, meta = (DisplayName = "Override Alpha Settings", EditCondition = "bOverrideWeightAlphaSettings"))
	FLandscapeTexturePatchAlphaSettings WeightPatchAlphaSettings;

	/** Allows to override the patch component's weightmap blend node */
	UPROPERTY(EditAnywhere, Category = WeightPatch, meta = (InlineEditConditionToggle))
	bool bOverrideBlendMode = false;

	/** Determines how the patch is combined with the previous state of the landscape (overrides the patch component's own blend node). */
	UPROPERTY(EditAnywhere, Category = WeightPatch, meta = (EditCondition = "bOverrideBlendMode"))
	ELandscapeTexturePatchBlendMode OverrideBlendMode = ELandscapeTexturePatchBlendMode::AlphaBlend;

	// TODO: We could support having different per-layer falloff modes and falloff amounts as well, as
	// additional override members. But probably better to wait to see if that is actually desired.

	bool bReinitializeOnNextRender = false;

	void SetSourceMode(ELandscapeTexturePatchSourceMode NewMode);
	void SetWeightPatchAlphaSourceMode(ELandscapeTexturePatchAlphaSourceMode InSourceMode, bool bInOverrideAlpha, bool bAlwaysMarkDirty = true);
	void SetWeightPatchTextureChannel(ELandscapeTexturePatchTextureChannel InTextureChannel, bool bAlwaysMarkDirty = true);
	void SetWeightPatchAlphaTextureChannel(ELandscapeTexturePatchTextureChannel InTextureChannel, bool bAlwaysMarkDirty = true);
	void SetWeightPatchTextureTransform(const FLandscapeTexturePatchTextureTransform& InTextureTransform, bool bAlwaysMarkDirty = true);
	void SetWeightPatchAlphaTextureTransform(const FLandscapeTexturePatchTextureTransform& InTextureTransform, bool bAlwaysMarkDirty = true);

#if WITH_EDITOR
	void TransitionSourceModeInternal(ELandscapeTexturePatchSourceMode OldMode, ELandscapeTexturePatchSourceMode NewMode);
#endif

	friend class ULandscapeTexturePatch;
};


UCLASS(MinimalAPI, Blueprintable, BlueprintType, ClassGroup = Landscape, meta = (BlueprintSpawnableComponent))
class ULandscapeTexturePatch : public ULandscapePatchComponent
{
	GENERATED_BODY()

public:

#if WITH_EDITOR
	// ILandscapeEditLayerRenderer, via ULandscapePatchComponent
	LANDSCAPEPATCH_API virtual void GetRendererStateInfo(const UE::Landscape::EditLayers::FMergeContext* InMergeContext,
		UE::Landscape::EditLayers::FEditLayerTargetTypeState& OutSupportedTargetTypeState,
		UE::Landscape::EditLayers::FEditLayerTargetTypeState& OutEnabledTargetTypeState,
		TArray<UE::Landscape::EditLayers::FTargetLayerGroup>& OutTargetLayerGroups) const override;
	LANDSCAPEPATCH_API virtual FString GetEditLayerRendererDebugName() const override;
	LANDSCAPEPATCH_API virtual TArray<UE::Landscape::EditLayers::FEditLayerRenderItem> GetRenderItems(const UE::Landscape::EditLayers::FMergeContext* InMergeContext) const override;
	LANDSCAPEPATCH_API virtual UE::Landscape::EditLayers::ERenderFlags GetRenderFlags(const UE::Landscape::EditLayers::FMergeContext* InMergeContext) const override;
	LANDSCAPEPATCH_API virtual bool CanGroupRenderLayerWith(TScriptInterface<ILandscapeEditLayerRenderer> InOtherRenderer) const override;
	LANDSCAPEPATCH_API virtual bool RenderLayer(UE::Landscape::EditLayers::FRenderParams& RenderParams, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder) override;
	LANDSCAPEPATCH_API virtual void BlendLayer(UE::Landscape::EditLayers::FRenderParams& RenderParams, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder) override;

	// ULandscapePatchComponent
	LANDSCAPEPATCH_API virtual bool CanAffectHeightmap() const override;
	LANDSCAPEPATCH_API virtual bool CanAffectWeightmap() const override;
	LANDSCAPEPATCH_API virtual bool CanAffectWeightmapLayer(const FName& InLayerName) const override;
	LANDSCAPEPATCH_API virtual bool CanAffectVisibilityLayer() const override;
	LANDSCAPEPATCH_API virtual void GetRenderDependencies(TSet<UObject*>& OutDependencies) const override;

	// UActorComponent
	LANDSCAPEPATCH_API virtual TStructOnScope<FActorComponentInstanceData> GetComponentInstanceData() const override;
	LANDSCAPEPATCH_API virtual void CheckForErrors() override;

	// UObject
	LANDSCAPEPATCH_API virtual void PostLoad() override;
	LANDSCAPEPATCH_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	LANDSCAPEPATCH_API virtual void Serialize(FArchive& Ar) override;

	/**
	 * Gets the transform from patch to world. The transform is based off of the component
	 * transform, but with rotation changed to align to the landscape, only using the yaw
	 * to rotate it relative to the landscape.
	 */
	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	LANDSCAPEPATCH_API virtual FTransform GetPatchToWorldTransform() const;

	/**
	 * Gives size in unscaled world coordinates (ie before applying patch transform) of the patch as measured 
	 * between the centers of the outermost pixels. This is the range across which bilinear interpolation
	 * always has correct values, so the area outside this center portion in the texture does not affect
	 * the landscape.
	 */
	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	virtual FVector2D GetUnscaledCoverage() const { return FVector2D(UnscaledPatchCoverage); }

	/**
	 * Set the patch coverage (see GetUnscaledCoverage for description).
	 */
	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	virtual void SetUnscaledCoverage(FVector2D Coverage) { UnscaledPatchCoverage = Coverage; }

	/**
	 * When using an internal texture, gives size in unscaled world coordinates of the patch in the world,
	 * based off of UnscaledCoverage and texture resolution (i.e., adds a half-pixel around UnscaledCoverage).
	 * Does not reflect the resolution of any used texture assets (if the source mode is texture asset for
	 * the height/weight patches).
	 */
	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	LANDSCAPEPATCH_API virtual FVector2D GetFullUnscaledWorldSize() const;

	/**
	 * Gets the size (in pixels) of the internal textures used by the patch. Does not reflect the resolution
	 * of any used texture assets (if the source mode is texture asset).
	 */
	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	virtual FVector2D GetResolution() const { return FVector2D(ResolutionX, ResolutionY); }

	/**
	 * Sets the resolution of the currently used internal texture or render target. Has no effect
	 * if the source mode is set to an external texture asset.
	 *
	 * @return true if successful.
	 */
	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	LANDSCAPEPATCH_API virtual void SetResolution(FVector2D ResolutionIn);

	/**
	 * Given the landscape resolution, current patch coverage, and a landscape resolution multiplier, gives the
	 * needed resolution of the landscape patch. I.e., figures out the number of pixels in the landcape that
	 * would be in a region of such size, and then uses the resolution multiplier to give a result.
	 *
	 * @return true if successful (may fail if landscape is not set, for instance)
	 */
	UFUNCTION(BlueprintCallable, Category = LandscapePatch, meta = (ResolutionMultiplier = "1.0"))
	LANDSCAPEPATCH_API virtual UPARAM(DisplayName = "Success") bool GetInitResolutionFromLandscape(float ResolutionMultiplier, FVector2D& ResolutionOut) const;

	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	float GetFalloff() const { return Falloff; }

	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	void SetFalloff(float FalloffIn) 
	{
		if (Falloff != FalloffIn)
		{
			Modify();
			Falloff = FalloffIn;
		}
	}

	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	ELandscapeTexturePatchFalloffMode GetFalloffMode() const { return FalloffMode; }

	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	void SetFalloffMode(ELandscapeTexturePatchFalloffMode FalloffModeIn) 
	{
		if (FalloffMode != FalloffModeIn)
		{
			Modify();
			FalloffMode = FalloffModeIn;
		}
	}

	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	ELandscapeTexturePatchBlendMode GetBlendMode() const { return BlendMode; }

	/**
	 * Determines how the height patch is blended into the existing terrain.
	 */
	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	void SetBlendMode(ELandscapeTexturePatchBlendMode BlendModeIn) 
	{
		if (BlendMode != BlendModeIn)
		{
			Modify();
			BlendMode = BlendModeIn;
		}
	}

	// Height related functions:

	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	virtual ELandscapeTexturePatchSourceMode GetHeightSourceMode() const { return HeightSourceMode; }

	/**
	 * Changes source mode. There are currently no API guarantees regarding the initialization of the
	 * new source data. E.g. when first switching to use an internal render target, the data in that
	 * render target may not be initialized.
	 */
	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	LANDSCAPEPATCH_API virtual void SetHeightSourceMode(ELandscapeTexturePatchSourceMode NewMode);

	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	virtual const FLandscapeTexturePatchTextureTransform& GetHeightTextureTransform() const { return HeightTextureUVTransform; };

	/**
	 * Sets the height texture's UV transform. Transform is applied relative UVSpace unless struct property bInWorldSpaceScale is true
	 * @param InTextureTransform the height texture's transform
	 * @param bAlwaysMarkDirty mark the package dirty when transform changes
	 */
	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	LANDSCAPEPATCH_API virtual void SetHeightTextureTransform(const FLandscapeTexturePatchTextureTransform& InTextureTransform, bool bAlwaysMarkDirty = true);

	/**
	 * Sets the height texture's color channel. Texture channel is restricted when HeightSource is InternalData or HeightEncoding is NativePackedHeight (Assumes RG channels)
	 * @param InTextureChannel the color channel of the texture to use
	 * @param bAlwaysMarkDirty mark the package dirty when channel property changes
	 */
	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	LANDSCAPEPATCH_API virtual void SetHeightTextureChannel(ELandscapeTexturePatchTextureChannel InTextureChannel, bool bAlwaysMarkDirty = true);

	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	virtual ELandscapeTexturePatchTextureChannel GetHeightTextureChannel() const { return HeightTextureChannel; }

	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	virtual ELandscapeTexturePatchAlphaSourceMode GetHeightAlphaSourceMode() const { return HeightAlphaSettings.GetAlphaSourceMode(); }

	/**
	 * Changes the source mode of the height patch alpha data. There are currently no API guarantees regarding the initialization of the
	 * new alpha source data. E.g. when first switching to use an internal render target, the data in that
	 * render target may not be initialized.
	 */
	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	LANDSCAPEPATCH_API virtual void SetHeightAlphaSourceMode(ELandscapeTexturePatchAlphaSourceMode InNewMode, bool bAlwaysMarkDirty = true);

	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	virtual const FLandscapeTexturePatchTextureTransform& GetHeightAlphaTextureTransform() const { return HeightAlphaSettings.AlphaTextureUVTransform; };

	/**
	 * Sets the height alpha texture's UV transform. Transform is applied relative UVSpace unless struct property bInWorldSpaceScale is true
	 * @param InTextureTransform the height alpha texture's transform
	 * @param bAlwaysMarkDirty mark the package dirty when alpha transform changes
	 */
	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	LANDSCAPEPATCH_API virtual void SetHeightAlphaTextureTransform(const FLandscapeTexturePatchTextureTransform& InTextureTransform, bool bAlwaysMarkDirty = true);

	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	virtual ELandscapeTexturePatchTextureChannel GetHeightAlphaSourceTextureChannel() const { return HeightAlphaSettings.AlphaTextureChannel; }

	 /**
	 * Sets the height alpha texture patch color channel
	 * @param InTextureChannel the color channel of the texture to use for the height alpha
	 * @param bAlwaysMarkDirty mark the package dirty when channel property changes
	 */
	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	LANDSCAPEPATCH_API virtual void SetHeightAlphaTextureChannel(ELandscapeTexturePatchTextureChannel InTextureChannel, bool bAlwaysMarkDirty = true);

	/**
	* Gets the internal height alpha render target, if the height alpha source mode is set to Texture Backed Render Target
	*
	* Things that should be set up if using the internal render target:
	* - An appropriate texture size should have been set with SetResolution. If the patch extent has already
	*  been set, you can base your resolution on the extent and the resolution of the landscape by using
	*  GetInitResolutionFromLandscape()
	* - The alpha uses RenderTargetFormat RTF_R8 by default. The AlphaTextureChannel property defines the single color channel 
	*   storing the Alpha data
	*
	* @param bMarkDirty If true, marks the containing package as dirty, since the render target is presumably
	*  being written to. Can be set to false if the render target is not being written to.
	*/
	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	LANDSCAPEPATCH_API virtual UTextureRenderTarget2D* GetHeightAlphaRenderTarget(bool bMarkDirty = true);

	/**
	 * Sets the texture used for height when the height source mode is set to texture asset. Note that
	 * virtual textures are not supported.
	 */
	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	LANDSCAPEPATCH_API void SetHeightTextureAsset(UTexture* TextureIn);

	/**
	 * Gets the internal height render target, if source mode is set to Texture Backed Render Target.
	 * 
	 * Things that should be set up if using the internal render target:
	 * - SetHeightSourceMode should have been called with TextureBackedRenderTarget.
	 * - An appropriate texture size should have been set with SetResolution. If the patch extent has already
	 *  been set, you can base your resolution on the extent and the resolution of the landscape by using
	 *  GetInitResolutionFromLandscape().
	 * - SetHeightRenderTargetFormat should have been called with a desired format. In particular, if using
	 *  an alpha channel, the format should have an alpha channel (and SetUseAlphaChannelForHeight should have
	 *  been called with "true").
	 * 
	 * In addition, you may need to call SetHeightEncodingMode, SetHeightEncodingSettings, and SetZeroHeightMeaning
	 * based on how you want the data you write to be interpreted. This part is not specific to using an internal render
	 * target, since you are likely to need to do that with a TextureAsset source mode as well.
	 * 
	 * @param bMarkDirty If true, marks the containing package as dirty, since the render target is presumably
	 *  being written to. Can be set to false if the render target is not being written to.
	 */
	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	LANDSCAPEPATCH_API virtual UTextureRenderTarget2D* GetHeightRenderTarget(bool bMarkDirty = true);

	UFUNCTION(BlueprintCallable, Category = LandscapePatch, meta = (ETextureRenderTargetFormat = "ETextureRenderTargetFormat::RTF_R32f"))
	LANDSCAPEPATCH_API void SetHeightRenderTargetFormat(ETextureRenderTargetFormat Format);

	UE_DEPRECATED(all, "SetUseAlphaChannelForHeight is deprecated. Use SetHeightAlphaSourceMode with ELandscapeTexturePatchAlphaSourceMode instead (use alpha mode None or SourceTextureChannel for legacy behavior)")
	UFUNCTION(meta = (DeprecatedFunction, DeprecationMessage = "SetUseAlphaChannelForHeight has been deprecated, use SetHeightAlphaSourceMode instead (use alpha mode None or SourceTextureChannel for legacy behavior)."))
	void SetUseAlphaChannelForHeight(bool bUse)
	{
		// Deprecated
		SetHeightAlphaSourceMode(bUse ? ELandscapeTexturePatchAlphaSourceMode::SourceTextureChannel : ELandscapeTexturePatchAlphaSourceMode::None);
	}

	/**
	 * Set the height encoding mode for the patch, which determines how stored values in the patch
	 * are translated into heights when applying to landscape.
	 */
	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	void SetHeightEncodingMode(ELandscapeTextureHeightPatchEncoding EncodingMode)
	{
		if (HeightEncoding != EncodingMode)
		{
			Modify();
			HeightEncoding = EncodingMode;

			// When the height uses internal texture / native packed height the texture channel is restricted (assume RG texture channels)
			if (HeightEncoding == ELandscapeTextureHeightPatchEncoding::NativePackedHeight)
			{
				HeightTextureChannel = ELandscapeTexturePatchTextureChannel::None;
			}
		}
	}

	/**
	 * Just like SetSourceEncodingMode, but resets ZeroInEncoding, WorldSpaceEncodingScale, and height
	 * render target format to mode-specific defaults.
	 */
	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	LANDSCAPEPATCH_API void ResetHeightEncodingMode(ELandscapeTextureHeightPatchEncoding EncodingMode);

	/**
	 * Set settings that determine how values in the patch are translated into heights. This is only
	 * used if the encoding mode is not NativePackedHeight, where values are expected to be already
	 * in the same space as the landscape heightmap.
	 */
	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	LANDSCAPEPATCH_API void SetHeightEncodingSettings(const FLandscapeTexturePatchEncodingSettings& Settings);

	/**
	 * Set how zero height is interpreted, see comments in ELandscapeTextureHeightPatchZeroHeightMeaning.
	 */
	UFUNCTION(BlueprintCallable, Category = "LandscapePatch")
	void SetZeroHeightMeaning(ELandscapeTextureHeightPatchZeroHeightMeaning ZeroHeightMeaningIn)
	{ 
		if (ZeroHeightMeaning != ZeroHeightMeaningIn)
		{
			Modify();
			ZeroHeightMeaning = ZeroHeightMeaningIn;
		}
	}


	// Weight related functions:
	UE_DEPRECATED(all, "Use CreateWeightPatch with ELandscapeTexturePatchAlphaSourceMode instead (use alpha mode None or SourceTextureChannel for legacy behavior)")
	UFUNCTION(meta = (DeprecatedFunction, DeprecationMessage = "AddWeightPatch has been deprecated, use CreateWeightPatch with ELandscapeTexturePatchAlphaSourceMode instead (use None or SourceTextureChannel for legacy behavior)."))
	LANDSCAPEPATCH_API virtual void AddWeightPatch(const FName& InWeightmapLayerName, ELandscapeTexturePatchSourceMode SourceMode, bool bUseAlphaChannel); 

	/**
	 * By default, the layer is added with source mode set to be a texture-backed render target.
	 */
	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	LANDSCAPEPATCH_API virtual void CreateWeightPatch(const FName& InWeightmapLayerName, ELandscapeTexturePatchSourceMode InSourceMode, ELandscapeTexturePatchAlphaSourceMode InAlphaSourceMode);

	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	LANDSCAPEPATCH_API virtual void RemoveWeightPatch(const FName& InWeightmapLayerName);

	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	LANDSCAPEPATCH_API virtual void RemoveAllWeightPatches();

	/** Sets the source mode of all weight patches to "None". */
	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	LANDSCAPEPATCH_API virtual void DisableAllWeightPatches();

	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	LANDSCAPEPATCH_API TArray<FName> GetAllWeightPatchLayerNames();
	
	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	LANDSCAPEPATCH_API virtual ELandscapeTexturePatchSourceMode GetWeightPatchSourceMode(const FName& InWeightmapLayerName);

	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	LANDSCAPEPATCH_API virtual const FLandscapeTexturePatchTextureTransform& GetWeightPatchTextureTransform(const FName& InWeightmapLayerName) const;

	/**
	 * Sets the weight texture's UV transform. Transform is applied relative UVSpace unless struct property bInWorldSpaceScale is true
	 * @param InWeightmapLayerName weightmap layer name to set the texture transform
	 * @param InTextureTransform the weight texture's transform
	 * @param bAlwaysMarkDirty mark the package dirty when transform changes
	 */
	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	LANDSCAPEPATCH_API virtual void SetWeightPatchTextureTransform(const FName& InWeightmapLayerName, const FLandscapeTexturePatchTextureTransform& InTextureTransform, bool bAlwaysMarkDirty = true);

	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	LANDSCAPEPATCH_API virtual UTexture* GetWeightPatchTextureAsset(const FName& InWeightmapLayerName);

	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	LANDSCAPEPATCH_API virtual void SetWeightPatchSourceMode(const FName& InWeightmapLayerName, ELandscapeTexturePatchSourceMode NewMode);

	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	LANDSCAPEPATCH_API virtual ELandscapeTexturePatchTextureChannel GetWeightPatchTextureChannel(const FName& InWeightmapLayerName) const;

	/**
	 * Sets the texture patch color channel specific weight patch
	 * @param InWeightmapLayerName weightmap layer name to set the texture channel
	 * @param InTextureChannel the channel of the texture the weight patch will use
	 * @param bAlwaysMarkDirty mark the package dirty when property changes
	 */
	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	LANDSCAPEPATCH_API virtual void SetWeightPatchTextureChannel(const FName& InWeightmapLayerName, ELandscapeTexturePatchTextureChannel InTextureChannel, bool bAlwaysMarkDirty = true);

	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	LANDSCAPEPATCH_API void SetWeightPatchTextureAsset(const FName& InWeightmapLayerName, UTexture* TextureIn);

	/**
	 * @param bMarkDirty If true, marks the containing package as dirty, since the render target is presumably
	 *  being written to. Can be set to false if the render target is not being written to.
	 */
	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	LANDSCAPEPATCH_API virtual UTextureRenderTarget2D* GetWeightPatchRenderTarget(const FName& InWeightmapLayerName, bool bMarkDirty = true);

	UE_DEPRECATED(all, "SetUseAlphaChannelForWeightPatch is deprecated. Use SetWeightPatchAlphaSourceMode with ELandscapeTexturePatchAlphaSourceMode instead (use None or SourceTextureChannel for legacy behavior)")
	UFUNCTION(meta = (DeprecatedFunction, DeprecationMessage = "SetUseAlphaChannelForWeightPatch has been deprecated, use SetWeightPatchAlphaSourceMode instead (use None or SourceTextureChannel for legacy behavior)."))
	LANDSCAPEPATCH_API virtual void SetUseAlphaChannelForWeightPatch(const FName& InWeightmapLayerName, bool bUseAlphaChannel); 

	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	LANDSCAPEPATCH_API virtual ELandscapeTexturePatchAlphaSourceMode GetWeightPatchAlphaSourceMode(const FName& InWeightmapLayerName) const;

	/**
	 * Sets the default weight patch alpha source mode for all weight patches or overrides the source for a specific weight patch
	 * @param InWeightmapLayerName weightmap layer name to set the alpha mode. Overrides a specific layer or sets default source mode when name is None
	 * @param InSourceMode the alpha source mode
	 * @param bInOverrideAlpha sets the weightmap layer's override flag. When false, the weightmap layer will fallback to the default source mode
 	 * @param bAlwaysMarkDirty mark the package dirty when property changes
	 */
	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	LANDSCAPEPATCH_API virtual void SetWeightPatchAlphaSourceMode(const FName& InWeightmapLayerName, ELandscapeTexturePatchAlphaSourceMode InSourceMode, bool bInOverrideAlpha = true, bool bAlwaysMarkDirty = true);

	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	LANDSCAPEPATCH_API virtual const UTexture* GetWeightPatchAlphaTextureAsset(const FName& InWeightmapLayerName) const;

	/**
 	 * Gets the internal weight patch alpha render target, if the weight patch alpha source mode is set to Texture Backed Render Target
	 * Returns a reference to the default (shared) weightmap alpha render target unless the given weightmap patch overrides the default alpha settings
	 *
	 * - The alpha uses RenderTargetFormat RTF_R8 by default. The AlphaTextureChannel property defines the single color channel 
	 *   storing the Alpha data
	 * 
	 * @param bMarkDirty If true, marks the containing package as dirty, since the render target is presumably
	 *  being written to. Can be set to false if the render target is not being written to.
	 */
	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	LANDSCAPEPATCH_API virtual UTextureRenderTarget2D* GetWeightPatchAlphaRenderTarget(const FName& InWeightmapLayerName, bool bMarkDirty = true);

	/**
	 * Sets the default weight patch alpha texture asset for all weight patches or overrides the texture for a specific weight patch
	 * @param InWeightmapLayerName weightmap layer name to set the alpha texture. Overrides a specific layer or sets the default texture when name is None
	 * @param InTexture the alpha texture the weightmap will use
	 * @param bInOverrideAlpha sets the weightmap layer's override flag. When false, the weightmap layer will fallback to the default alpha texture
	 * @param bAlwaysMarkDirty mark the package dirty when property changes
	 */
	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	LANDSCAPEPATCH_API void SetWeightPatchAlphaTextureAsset(const FName& InWeightmapLayerName, UTexture* InTexture, bool bInOverrideAlpha = true, bool bAlwaysMarkDirty = true);

	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	LANDSCAPEPATCH_API virtual const FLandscapeTexturePatchTextureTransform& GetWeightPatchAlphaTextureTransform(const FName& InWeightmapLayerName) const;

	/**
	 * Sets the weight texture's alpha UV transform. Transform is applied relative UVSpace unless struct property bInWorldSpaceScale is true
	 * @param InWeightmapLayerName weightmap layer name to set the texture transform
	 * @param InTextureTransform the weight alpha texture's transform
	 * @param bAlwaysMarkDirty mark the package dirty when alpha transform changes
	 */
	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	LANDSCAPEPATCH_API virtual void SetWeightPatchAlphaTextureTransform(const FName& InWeightmapLayerName, const FLandscapeTexturePatchTextureTransform& InTextureTransform, bool bAlwaysMarkDirty = true);

	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	LANDSCAPEPATCH_API virtual ELandscapeTexturePatchTextureChannel GetWeightPatchAlphaTextureChannel(const FName& InWeightmapLayerName) const;

	/**
	 * Sets the texture patch color channel for all weight patches or set the texture channel for a specific weight patch alpha override
	 * @param InWeightmapLayerName weightmap layer name to set the alpha texture channel. Overrides a specific layer or sets to the default texture channel when name is None
	 * @param InTextureChannel the channel of the texture the alpha mask will use
	 * @param bInOverrideAlpha sets the weightmap layer's override flag. When false, the weightmap layer will fallback to the default alpha channel
	 * @param bAlwaysMarkDirty mark the package dirty when property changes
	 */
	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	LANDSCAPEPATCH_API virtual void SetWeightPatchAlphaTextureChannel(const FName& InWeightmapLayerName, ELandscapeTexturePatchTextureChannel InTextureChannel, bool bInOverrideAlpha = true, bool bAlwaysMarkDirty = true);

	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	LANDSCAPEPATCH_API virtual void SetWeightPatchBlendModeOverride(const FName& InWeightmapLayerName, ELandscapeTexturePatchBlendMode BlendMode);

	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	LANDSCAPEPATCH_API virtual void ClearWeightPatchBlendModeOverride(const FName& InWeightmapLayerName);

	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	LANDSCAPEPATCH_API virtual void SetEditVisibilityLayer(const FName& InWeightmapLayerName, const bool bEditVisibilityLayer);

protected:
	//~ Don't expose these on the instance because a user might not realize that they would lose their existing internal
	//~ data by dragging them, and the only way they can reinitialize data in the viewport is through the methods that
	//~ already use InitTextureSizeX/Y as inputs
	UPROPERTY(EditDefaultsOnly, Category = Settings)
	int32 ResolutionX = 32;
	UPROPERTY(EditDefaultsOnly, Category = Settings)
	int32 ResolutionY = 32;

	/** At scale 1.0, the X and Y of the region affected by the height patch. This corresponds to the distance from the center
	 of the first pixel to the center of the last pixel in the patch texture in the X and Y directions. */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (UIMin = "0", ClampMin = "0"))
	FVector2D UnscaledPatchCoverage = FVector2D(2000, 2000);

	/** Determines how the patch's heightmap is combined with the previous state of the landscape. */
	UPROPERTY(EditAnywhere, Category = Settings)
	ELandscapeTexturePatchBlendMode BlendMode = ELandscapeTexturePatchBlendMode::AlphaBlend;

	UPROPERTY(EditAnywhere, Category = Settings)
	ELandscapeTexturePatchFalloffMode FalloffMode = ELandscapeTexturePatchFalloffMode::RoundedRectangle;

	/**
	 * Distance (in unscaled world coordinates) across which to smoothly fall off the patch effects.
	 */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin = "0", UIMax = "2000"))
	float Falloff = 0;


	// Height properties:

	// How the heightmap of the patch is stored. This is the property that is actually used, and it will
	// agree with DetailPanelHeightSourceMode at all times except when user is changing the latter via the
	// detail panel.
	//~ TODO: The property specifiers here are a hack to force this (hidden) property to be preserved across reruns of
	//~ a construction script in a blueprint actor. We should find the proper way that this is supposed to be done.
	UPROPERTY(EditAnywhere, Category = HeightPatch, meta = (EditConditionHides, EditCondition = "false"))
	ELandscapeTexturePatchSourceMode HeightSourceMode = ELandscapeTexturePatchSourceMode::None;

	/**
	 * How the heightmap of the patch is stored.
	 */
	UPROPERTY(EditAnywhere, Category = HeightPatch, meta = (DisplayName = "Source Mode"))
	ELandscapeTexturePatchSourceMode DetailPanelHeightSourceMode = ELandscapeTexturePatchSourceMode::None;

	/** Texture's color channel used for the height. The texture's Red channel is selected by default. 
	* The patch will not affect the landscape when no channel is selected.
	* When the height source mode is InternalTexture or HeightEncoding is NativePackedHeight the texture channel cannot be customized (assumes RG channel)
	*/
	UPROPERTY(EditAnywhere, Category = HeightPatch, meta = (DisplayName = "Texture Channels", EditConditionHides, 
		EditCondition = "HeightSourceMode != ELandscapeTexturePatchSourceMode::None && HeightSourceMode != ELandscapeTexturePatchSourceMode::InternalTexture && HeightEncoding != ELandscapeTextureHeightPatchEncoding::NativePackedHeight"))
	ELandscapeTexturePatchTextureChannel HeightTextureChannel = ELandscapeTexturePatchTextureChannel::Red;

	/** Not directly settable via detail panel- for display/debugging purposes only. */
	UPROPERTY(VisibleAnywhere, Category = HeightPatch, Instanced, AdvancedDisplay, meta = (DisplayName = "Debug Data"))
	TObjectPtr<ULandscapeHeightTextureBackedRenderTarget> HeightInternalData = nullptr;

	/**
	 * Texture used when source mode is set to a texture asset.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = HeightPatch, meta = (EditConditionHides,
		EditCondition = "HeightSourceMode == ELandscapeTexturePatchSourceMode::TextureAsset", 
		DisallowedAssetDataTags = "VirtualTextureStreaming=True"))
	TObjectPtr<UTexture> HeightTextureAsset = nullptr;

	UPROPERTY(EditAnywhere, Category = HeightPatch, meta = (DisplayName = "Texture Transform"), meta = (EditConditionHides, EditCondition = "HeightSourceMode != ELandscapeTexturePatchSourceMode::None"))
	FLandscapeTexturePatchTextureTransform HeightTextureUVTransform;

	UE_DEPRECATED(5.8, "bUseAlphaChannel is deprecated, HeightAlphaSettings.SetAlphaSourceMode() with ELandscapeTexturePatchAlphaSourceMode (use None or SourceTextureChannel for legacy behavior).")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use HeightAlphaSettings.SetAlphaSourceMode() with ELandscapeTexturePatchAlphaSourceMode instead (use None or SourceTextureChannel for legacy behavior)"))
	bool bUseTextureAlphaForHeight_DEPRECATED = false;

	UPROPERTY(EditAnywhere, Category = HeightPatch, meta = (EditConditionHides, EditCondition = "HeightSourceMode != ELandscapeTexturePatchSourceMode::None"))
	FLandscapeTexturePatchAlphaSettings HeightAlphaSettings;

	/** How the values stored in the patch represent the height. Not customizable for Internal Texture source mode, which always uses native packed height. */
	UPROPERTY(EditAnywhere, Category = HeightPatch, meta = (
		EditCondition = "HeightSourceMode != ELandscapeTexturePatchSourceMode::InternalTexture"))
	ELandscapeTextureHeightPatchEncoding HeightEncoding = ELandscapeTextureHeightPatchEncoding::WorldUnits;

	/** Encoding settings. Not relevant when using native packed height as the encoding. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = HeightPatch, meta = (UIMin = "0", UIMax = "1",
		EditCondition = "HeightSourceMode != ELandscapeTexturePatchSourceMode::InternalTexture && HeightEncoding != ELandscapeTextureHeightPatchEncoding::NativePackedHeight"))
	FLandscapeTexturePatchEncodingSettings HeightEncodingSettings;

	/**
	 * How 0 height is interpreted.
	 */
	UPROPERTY(EditAnywhere, Category = HeightPatch)
	ELandscapeTextureHeightPatchZeroHeightMeaning ZeroHeightMeaning = ELandscapeTextureHeightPatchZeroHeightMeaning::PatchZ;

	/**
	 * Whether to apply the patch Z scale to the height stored in the patch.
	 */
	UPROPERTY(EditAnywhere, Category = HeightPatch, AdvancedDisplay, meta = (DisplayName = "Apply Component Z Scale"))
	bool bApplyComponentZScale = true;


	// Weight properties:

	/** 
	 * Weight patches. 
	 * Note that manipulating these in the blueprint editor will not reliably update instances that are already
	 * placed into the world, due to current limitations in how change detection is done for such arrays. Specifically,
	 * existing instances that are actually not customized are very likely to be erroneously be treated as having
	 * customized their version of the array, causing the blueprint changes to not be pushed to those instances
	 * when they otherwise would be for most other properties.
	 */
	UPROPERTY(EditAnywhere, Category = WeightPatches, Instanced, NoClear, meta=(NoResetToDefault))
	TArray<TObjectPtr<ULandscapeWeightPatchTextureInfo>> WeightPatches;

	/* Default weight patches alpha settings.
	* Defines the alpha mask settings for all weight patches within this texture patch. Alpha is only applied
	* to weight patches with a valid source mode. Alpha settings can be overridden by individual weight patches.
	*/
	UPROPERTY(EditAnywhere, Category = WeightPatches)
	FLandscapeTexturePatchAlphaSettings WeightPatchesAlphaSettings;

	/** Determines how the patch's weightmap layers are combined with the previous state of the landscape. Blend
	* mode can be overriden by individual weight patches */
	UPROPERTY(EditAnywhere, Category = WeightPatches)
	ELandscapeTexturePatchBlendMode WeightPatchesBlendMode = ELandscapeTexturePatchBlendMode::AlphaBlend;

	// Reinitialization from detail panel:

	/**
	 * Given the current initialization settings, reinitialize the height patch.
	 */
	UFUNCTION(CallInEditor, Category = HeightPatch, meta = (DisplayName= "Reinitialize Height"))
	void RequestReinitializeHeight();

	UFUNCTION(CallInEditor, Category = WeightPatches, meta = (DisplayName = "Reinitialize Weights"))
	void RequestReinitializeWeights();

	bool bReinitializeHeightOnNextRender = false;

	/**
	 * Adjusts patch rotation to be aligned to a 90 degree increment relative to the landscape,
	 * adjusts UnscaledPatchCoverage such that it becomes a multiple of landscape quad size, and
	 * adjusts patch location so that the boundaries of the covered area lie on the nearest
	 * landscape vertices.
	 * Note that this doesn't adjust the resolution of the texture that the patch uses, so landscape
	 * vertices within the inside of the patch may still not always align with texture patch pixel
	 * centers (if the resolutions aren't multiples of each other).
	 */
	UFUNCTION(BlueprintCallable, CallInEditor, Category = Initialization)
	void SnapToLandscape();

	/** When initializing from landscape, set resolution based off of the landscape (and a multiplier). */
	UPROPERTY(EditAnywhere, Category = Initialization)
	bool bBaseResolutionOffLandscape = true;

	/** 
	 * Multiplier to apply to landscape resolution when initializing patch resolution. A value greater than 1.0 will use higher
	 * resolution than the landscape (perhaps useful for slightly more accurate results while not aligned to landscape), and
	 * a value less that 1.0 will use lower.
	 */
	UPROPERTY(EditAnywhere, Category = Initialization, meta = (EditCondition = "bBaseResolutionOffLandscape"))
	float ResolutionMultiplier = 1;

	/** Texture width to use when reinitializing using Reinitialize Weights or ReinitializeHeight, if not basing resolution off landscape. */
	UPROPERTY(EditAnywhere, Category = Initialization, meta = (EditCondition = "!bBaseResolutionOffLandscape", ClampMin = "1"))
	int32 InitTextureSizeX = 33;

	/** Texture height to use when reinitializing using Reinitialize Weights or ReinitializeHeight, if not basing resolution off landscape. */
	UPROPERTY(EditAnywhere, Category = Initialization, meta = (EditCondition = "!bBaseResolutionOffLandscape", ClampMin = "1"))
	int32 InitTextureSizeY = 33;

	LANDSCAPEPATCH_API UTexture2D* GetHeightInternalTexture();
	LANDSCAPEPATCH_API UTextureRenderTarget2D* GetHeightAlphaRenderTarget();
	LANDSCAPEPATCH_API UTexture2D* GetHeightAlphaInternalTexture();

	LANDSCAPEPATCH_API UTextureRenderTarget2D* GetWeightPatchRenderTarget(ULandscapeWeightPatchTextureInfo* WeightPatch);
	LANDSCAPEPATCH_API UTexture2D* GetWeightPatchInternalTexture(ULandscapeWeightPatchTextureInfo* WeightPatch);
	/** Returns the default weight patch alpha render target or the specific weight patch override render target */
	LANDSCAPEPATCH_API UTextureRenderTarget2D* GetWeightPatchAlphaRenderTarget(ULandscapeWeightPatchTextureInfo* WeightPatch, bool bMarkDirty = true);
	/** Returns the default weight patch alpha internal texture or the specific weight patch override internal texture */
	LANDSCAPEPATCH_API UTexture2D* GetWeightPatchAlphaInternalTexture(ULandscapeWeightPatchTextureInfo* WeightPatch, bool bMarkDirty = true);

private:
	void UpdateHeightConvertToNativeParamsIfNeeded();

	 /**
	 * If the weight patch overrides the alpha, returns the specific weight patch override settings. Otherwise, returns the default weight patch alpha settings 
	 */
	FLandscapeTexturePatchAlphaSettings& GetWeightPatchAlphaSettings(ULandscapeWeightPatchTextureInfo* WeightPatch);
	const FLandscapeTexturePatchAlphaSettings& GetWeightPatchAlphaSettingsConst(ULandscapeWeightPatchTextureInfo* WeightPatch) const; 

	TObjectPtr<ULandscapeWeightPatchTextureInfo> GetWeightPatch(const FName& InWeightmapLayerName, const TCHAR* InDebugLogName = TEXT("ULandscapeTexturePatch::GetWeightPatch")) const;

#if WITH_EDITOR
	void TransitionHeightSourceModeInternal(ELandscapeTexturePatchSourceMode OldMode, ELandscapeTexturePatchSourceMode NewMode);
	FLandscapeHeightPatchConvertToNativeParams GetHeightConvertToNativeParams() const;

	UTextureRenderTarget2D* ApplyToHeightmap(UE::Landscape::EditLayers::FRenderParams* RenderParams, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder, UTextureRenderTarget2D* InCombinedResult,
		const FTransform& LandscapeHeightmapToWorld, bool& bHasRenderedSomething, ERHIAccess OutputAccess = ERHIAccess::None);

	void ApplyToWeightmap(UE::Landscape::EditLayers::FRenderParams* RenderParams, UE::Landscape::FRDGBuilderRecorder& RDGBuilderRecorder, ULandscapeWeightPatchTextureInfo* PatchInfo,
		FTextureResource* InMergedLandscapeTextureResource, int32 LandscapeTextureSliceIndex, const FIntPoint& LandscapeTextureResolution, 
		const FTransform& LandscapeHeightmapToWorld, bool& bHasRenderedSomething, ERHIAccess OutputAccess = ERHIAccess::None);

	void GetCommonShaderParams(const FTransform& LandscapeHeightmapToWorldIn,
		const FIntPoint& SourceResolutionIn, const FIntPoint& DestinationResolutionIn,
		const FLandscapeTexturePatchTextureTransform& SourceTextureTransformIn, const FLandscapeTexturePatchTextureTransform& AlphaTextureTransformIn,
		FTransform& PatchToWorldOut, FVector2f& PatchWorldDimensionsOut, FMatrix44f& HeightmapToPatchUVOut, FMatrix44f& HeightmapToPatchUVTransformOut,
		FMatrix44f& HeightmapAlphaToPatchUVTransformOut,
		FIntRect& DestinationBoundsOut, FVector2f& EdgeUVDeadBorderOut, float& FalloffWorldMarginOut) const;
	void GetHeightShaderParams(const FTransform& LandscapeHeightmapToWorldIn, 
		const FIntPoint& SourceResolutionIn, const FIntPoint& DestinationResolutionIn,
		UE::Landscape::FApplyLandscapeTextureHeightPatchPSParameters& ParamsOut, FIntRect& DestinationBoundsOut) const;
	void GetWeightShaderParams(const FTransform& LandscapeHeightmapToWorldIn, const FIntPoint& SourceResolutionIn, 
		const FIntPoint& DestinationResolutionIn, const ULandscapeWeightPatchTextureInfo* WeightPatchInfo, 
		UE::Landscape::FApplyLandscapeTextureWeightPatchPSParameters& ParamsOut, FIntRect& DestinationBoundsOut) const;
	FMatrix44f GetPatchToHeightmapUVs(const FTransform& LandscapeHeightmapToWorld, int32 PatchSizeX, int32 PatchSizeY, int32 HeightmapSizeX, int32 HeightmapSizeY) const;
	void ReinitializeHeight(UTextureRenderTarget2D* InCombinedResult, const FTransform& LandscapeHeightmapToWorld);
	/**
	 * @param SliceIndex set to a negative value when not using a Texture2DArray
	 */
	void ReinitializeWeightPatch(ULandscapeWeightPatchTextureInfo* PatchInfo,
		FTextureResource* InputResource, FIntPoint ResourceSize, int32 SliceIndex,
		const FTransform& LandscapeHeightmapToWorld);

	bool WeightPatchCanRender(const ULandscapeWeightPatchTextureInfo& InWeightPatch) const;

	void ResetHeightRenderTargetFormat();
#endif // WITH_EDITOR

	UPROPERTY(EditDefaultsOnly, Category = Settings)
	TEnumAsByte<ETextureRenderTargetFormat> HeightRenderTargetFormat = ETextureRenderTargetFormat::RTF_R32f;
};
