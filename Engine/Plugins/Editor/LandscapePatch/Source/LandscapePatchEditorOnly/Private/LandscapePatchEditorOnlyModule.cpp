// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapePatchEditorOnlyModule.h"

#include "Editor/UnrealEdEngine.h"
#include "LandscapePatchUIStyle.h"
#include "LandscapeTexturePatch.h"
#include "LandscapeTexturePatchVisualizer.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "UnrealEdGlobals.h"

#define LOCTEXT_NAMESPACE "FLandscapePatchEditorOnlyModule"

void FLandscapePatchEditorOnlyModule::StartupModule()
{
	FLandscapePatchUIStyle::Initialize();

	if (GUnrealEd)
	{

		TSharedPtr<FLandscapeTexturePatchVisualizer> Visualizer = MakeShared<FLandscapeTexturePatchVisualizer>();
		GUnrealEd->RegisterComponentVisualizer(ULandscapeTexturePatch::StaticClass()->GetFName(), Visualizer);
		// This call should maybe be inside the RegisterComponentVisualizer call above, but since it's not,
		// we'll put it here.
		Visualizer->OnRegister();
		VisualizersToUnregisterOnShutdown.Add(ULandscapeTexturePatch::StaticClass()->GetFName());
	}
}

void FLandscapePatchEditorOnlyModule::ShutdownModule()
{
	if (GUnrealEd)
	{
		for (const FName& Name : VisualizersToUnregisterOnShutdown)
		{
			GUnrealEd->UnregisterComponentVisualizer(Name);
		}
	}

	// Unregister customizations
	FPropertyEditorModule* PropertyEditorModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor");
	if (PropertyEditorModule)
	{
		for (const FName& ClassName : ClassesToUnregisterOnShutdown)
		{
			PropertyEditorModule->UnregisterCustomClassLayout(ClassName);
		}
	}

	FLandscapePatchUIStyle::Shutdown();
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FLandscapePatchEditorOnlyModule, LandscapePatchEditorOnly)
