// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapePatchUIStyle.h"
#include "Brushes/SlateImageBrush.h"
#include "Styling/SlateStyleRegistry.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleMacros.h"

#define RootToContentDir StyleSet->RootToContentDir

TSharedPtr<FSlateStyleSet> FLandscapePatchUIStyle::StyleSet;
TSharedPtr<class ISlateStyle> FLandscapePatchUIStyle::Get() { return StyleSet; }

FName FLandscapePatchUIStyle::GetStyleSetName()
{
	static FName LandscapePatchStyleName(TEXT("LandscapePatchUIStyle"));
	return LandscapePatchStyleName;
}

void FLandscapePatchUIStyle::Initialize()
{
	// Const icon sizes
	const FVector2D Icon8x8(8.0f, 8.0f);
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon40x40(40.0f, 40.0f);
	const FVector2D Icon64x64(64.0f, 64.0f);

	// Only register once
	if (StyleSet.IsValid())
	{
		return;
	}

	StyleSet = MakeShareable(new FSlateStyleSet(GetStyleSetName()));

	StyleSet->SetContentRoot(IPluginManager::Get().FindPlugin(TEXT("LandscapePatch"))->GetContentDir());
	StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	// LandscapePatchEditLayer
	StyleSet->Set("ClassIcon.LandscapePatchEditLayer", new IMAGE_BRUSH_SVG("Icons/LandscapePatchEditLayer_16", Icon16x16));
	StyleSet->Set("ClassThumbnail.LandscapePatchEditLayer", new IMAGE_BRUSH_SVG("Icons/LandscapePatchEditLayer_64", Icon64x64));

	// LandscapePatchComponent
	StyleSet->Set("ClassIcon.LandscapePatchComponent", new IMAGE_BRUSH_SVG("Icons/LandscapePatchComponent_16", Icon16x16));
	StyleSet->Set("ClassThumbnail.LandscapePatchComponent", new IMAGE_BRUSH_SVG("Icons/LandscapePatchComponent_64", Icon64x64));

	FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());
};

#undef RootToContentDir

void FLandscapePatchUIStyle::Shutdown()
{
	if (StyleSet.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
		ensure(StyleSet.IsUnique());
		StyleSet.Reset();
	}
}
