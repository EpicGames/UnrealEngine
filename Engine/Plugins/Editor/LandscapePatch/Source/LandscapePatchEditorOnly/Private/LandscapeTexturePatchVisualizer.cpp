// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeTexturePatchVisualizer.h"

#include "LandscapeTexturePatch.h"

#include "PrimitiveDrawingUtils.h" // FPrimitiveDrawInterface

void FLandscapeTexturePatchVisualizer::DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	FColor Color = FColor::Red;
	float Thickness = 3;
	float DepthBias = 1;
	bool bScreenSpace = 1;

	if (const ULandscapeTexturePatch* Patch = Cast<ULandscapeTexturePatch>(Component); (Patch != nullptr) && Patch->IsEnabled())
	{
		FTransform PatchToWorld = Patch->GetPatchToWorldTransform();
		DrawRectangle(PDI, PatchToWorld.GetTranslation(), PatchToWorld.GetUnitAxis(EAxis::X), PatchToWorld.GetUnitAxis(EAxis::Y),
			Color, Patch->GetUnscaledCoverage().X * PatchToWorld.GetScale3D().X, Patch->GetUnscaledCoverage().Y * PatchToWorld.GetScale3D().Y, SDPG_Foreground,
			Thickness, DepthBias, bScreenSpace);
	}
	// Note: when patches are deleted, selection seems to be updated only after giving another call to the
	// visualization drawing, so do not ensure if patch was null.
}
