// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "ComponentVisualizer.h"

#define UE_API LANDSCAPEPATCHEDITORONLY_API

/**
 * Adds a rectangle denoting the area covered by a landscape texture patch when it is selected in editor.
 */
class FLandscapeTexturePatchVisualizer : public FComponentVisualizer
{
private:
	UE_API virtual void DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI) override;
};

#undef UE_API
