// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Modules/ModuleInterface.h"

class FLandscapePatchEditorOnlyModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	/** StaticClass is not safe on shutdown, so we cache the name, and use this to unregister on shut down */
	TArray<FName> ClassesToUnregisterOnShutdown;
	TArray<FName> VisualizersToUnregisterOnShutdown;
};
